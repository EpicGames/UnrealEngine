// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStaticMeshEditorViewport.h"

#include "AdvancedPreviewSceneMenus.h"
#include "StaticMeshViewportLODCommands.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/Package.h"
#include "Components/StaticMeshComponent.h"
#include "Styling/AppStyle.h"
#include "Engine/StaticMesh.h"
#include "SceneInterface.h"
#include "IStaticMeshEditor.h"
#include "StaticMeshEditorActions.h"
#include "Slate/SceneViewport.h"
#include "ComponentReregisterContext.h"
#include "AnalyticsEventAttribute.h"
#include "AssetViewerSettings.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Widgets/Docking/SDockTab.h"
#include "Engine/StaticMeshSocket.h"
#include "SEditorViewportToolBarMenu.h"
#include "Editor.h"
#include "PreviewProfileController.h"
#include "StaticMeshEditor.h"
#include "StaticMeshEditorViewportToolbarSections.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "StaticMeshResources.h"
#include "SStaticMeshLODVisualizerWidget.h"

#define HITPROXY_SOCKET	1

#define LOCTEXT_NAMESPACE "StaticMeshEditorViewport"

///////////////////////////////////////////////////////////
// SStaticMeshEditorViewport

namespace StaticMeshEditorViewportLocals
{
	static TAutoConsoleVariable<int32> CVarEnableStaticMeshEditorAspectRatioControl(
		TEXT("StaticMeshEditor.EnableAspectRatioControls"),
		0,
		TEXT("Enable a new menu allowing for the fixing of aspect ratio in the Static Mesh Editor's viewport."));

	static TAutoConsoleVariable<int32> CVarEnableStaticMeshEditorLODVisualizer(
		TEXT("StaticMeshEditor.EnableLODVisualizer"),
		0,
		TEXT("Enable a new widget in the static mesh editor viewport to display the status of the current LOD."));
}

void SStaticMeshEditorViewport::Construct(const FArguments& InArgs)
{
	//PreviewScene = new FAdvancedPreviewScene(FPreviewScene::ConstructionValues(), 

	StaticMeshEditorPtr = InArgs._StaticMeshEditor;

	TSharedPtr<IStaticMeshEditor> PinnedEditor = StaticMeshEditorPtr.Pin();
	StaticMesh = PinnedEditor.IsValid() ? PinnedEditor->GetStaticMesh() : nullptr;

	if (StaticMesh)
	{
		PreviewScene->SetFloorOffset(static_cast<float>( -StaticMesh->GetExtendedBounds().Origin.Z + StaticMesh->GetExtendedBounds().BoxExtent.Z ));
	}

	// restore last used feature level
	UWorld* World = PreviewScene->GetWorld();
	if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine);
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			PreviewScene->GetWorld()->ShaderPlatformChanged(NewShaderPlatform);
		});

	CurrentViewMode = VMI_Lit;

	FStaticMeshViewportLODCommands::Register();

	SEditorViewport::Construct( SEditorViewport::FArguments() );

	PreviewMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient );
	ERHIFeatureLevel::Type FeatureLevel = GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel();
	if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		PreviewMeshComponent->SetMobility(EComponentMobility::Static);
	}
	SetPreviewMesh(StaticMesh);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SStaticMeshEditorViewport::OnObjectPropertyChanged);
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(PreviewScene, EditorViewportClient);

	SetAspectRatioState(EStaticMeshEditorAspectRatio::EPlatform);
}

void SStaticMeshEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(6.f)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::Get().GetBrush( "FloatingBorder" ) )
			.Padding(4.f)
			[
				SAssignNew(OverlayText, SRichTextBlock)
			]
		];

	// this widget will display the current viewed feature level
	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Left)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];

	if (StaticMeshEditorViewportLocals::CVarEnableStaticMeshEditorLODVisualizer.GetValueOnGameThread())
	{
		Overlay->AddSlot()
			.VAlign(EVerticalAlignment::VAlign_Fill)
			.HAlign(HAlign_Right)
			.Padding(5.0f)
			[
				SAssignNew(LODVisualizer, SStaticMeshLODVisualizerWidget)
			];
	}
	
}

SStaticMeshEditorViewport::SStaticMeshEditorViewport()
	: PreviewScene(MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues())))
{

}

SStaticMeshEditorViewport::~SStaticMeshEditorViewport()
{
	CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void SStaticMeshEditorViewport::PopulateOverlayText(const TArrayView<FOverlayTextItem> TextItems)
{
	FTextBuilder FinalText;

	static FText WarningTextStyle = FText::FromString(TEXT("<TextBlock.ShadowedTextWarning>{0}</>"));
	static FText NormalTextStyle = FText::FromString(TEXT("<TextBlock.ShadowedText>{0}</>"));

	for (const auto& TextItem : TextItems)
	{
		if (!TextItem.bIsCustomFormat)
		{
			FinalText.AppendLineFormat(TextItem.bIsWarning ? WarningTextStyle : NormalTextStyle, TextItem.Text);
		}
		else
		{
			FinalText.AppendLine(TextItem.Text);
		}
	}

	OverlayText->SetText(FinalText.ToText());
}

TSharedRef<SEditorViewport> SStaticMeshEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SStaticMeshEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SStaticMeshEditorViewport::OnFloatingButtonClicked()
{
}

void SStaticMeshEditorViewport::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( PreviewMeshComponent );
	Collector.AddReferencedObject( StaticMesh );
	Collector.AddReferencedObjects( SocketPreviewMeshComponents );
}

void SStaticMeshEditorViewport::RefreshViewport()
{
	// Invalidate the viewport's display.
	SceneViewport->Invalidate();

	if (LODVisualizer)
	{
		int32 NumLODs = GetLODCount();
		int32 CurrentLOD = GetCurrentLOD();

#if WITH_EDITORONLY_DATA && WITH_EDITOR
		FName PlatformName;
		// Lookup the override preview platform info, if any
		// @todo this doesn't set PlatformName, just a group, but GetValueForPlatform() will technically work being given a Group name instead of a platform name, so we just use it
		if (UObject::OnGetPreviewPlatform.IsBound() && UObject::OnGetPreviewPlatform.Execute(PlatformName))
		{
			LODVisualizer->SetCurrentPlatform(PlatformName.ToString());
		}		
#endif

		LODVisualizer->SetNumLODs(NumLODs);

		LODVisualizer->SetActiveLOD(CurrentLOD);
		LODVisualizer->SetMinimumLOD(GetCurrentMinimumLOD());

		LODVisualizer->SetNaniteEnabled(IsShowNaniteFallbackVisible());
		LODVisualizer->SetNaniteActive(!IsShowNaniteFallbackChecked());

		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			LODVisualizer->SetLODPercentage(LODIndex, GetScreenPercentageForLOD(LODIndex));
		}
	}
}

void SStaticMeshEditorViewport::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if ( !ensure(ObjectBeingModified) )
	{
		return;
	}

	if( PreviewMeshComponent )
	{
		bool bShouldUpdatePreviewSocketMeshes = (ObjectBeingModified == PreviewMeshComponent->GetStaticMesh());
		if( !bShouldUpdatePreviewSocketMeshes && PreviewMeshComponent->GetStaticMesh())
		{
			const int32 SocketCount = PreviewMeshComponent->GetStaticMesh()->Sockets.Num();
			for( int32 i = 0; i < SocketCount; ++i )
			{
				if( ObjectBeingModified == PreviewMeshComponent->GetStaticMesh()->Sockets[i] )
				{
					bShouldUpdatePreviewSocketMeshes = true;
					break;
				}
			}
		}

		if( bShouldUpdatePreviewSocketMeshes )
		{
			UpdatePreviewSocketMeshes();
			RefreshViewport();
		}
	}
}

bool SStaticMeshEditorViewport::PreviewComponentSelectionOverride(const UPrimitiveComponent* InComponent) const
{
	if (InComponent == PreviewMeshComponent)
	{
		const UStaticMeshComponent* Component = CastChecked<UStaticMeshComponent>(InComponent);
		return (Component->SelectedEditorSection != INDEX_NONE || Component->SelectedEditorMaterial != INDEX_NONE);
	}

	return false;
}

void SStaticMeshEditorViewport::ToggleShowNaniteFallback()
{
	if (PreviewMeshComponent)
	{
		FComponentReregisterContext ReregisterContext(PreviewMeshComponent);
		const bool bToggleOn = !PreviewMeshComponent->bDisplayNaniteFallbackMesh;
		PreviewMeshComponent->bDisplayNaniteFallbackMesh = bToggleOn;

		if (bToggleOn && IsShowRayTracingFallbackChecked())
		{
			ToggleShowRayTracingFallback();
		}
	}

	if (LODVisualizer)
	{
		LODVisualizer->SetNaniteEnabled(IsShowNaniteFallbackVisible());
		LODVisualizer->SetNaniteActive(!IsShowNaniteFallbackChecked());
	}

}

bool SStaticMeshEditorViewport::IsShowNaniteFallbackChecked() const
{
	return PreviewMeshComponent ? PreviewMeshComponent->bDisplayNaniteFallbackMesh : false;
}

bool SStaticMeshEditorViewport::IsShowNaniteFallbackVisible() const
{
	const UStaticMesh* PreviewStaticMesh = PreviewMeshComponent ? ToRawPtr(PreviewMeshComponent->GetStaticMesh()) : nullptr;

	return PreviewStaticMesh && PreviewStaticMesh->IsNaniteEnabled() ? true : false;
}

void SStaticMeshEditorViewport::ToggleShowDistanceField()
{
	if (EditorViewportClient)
	{
		const bool bToggleOn = !EditorViewportClient->EngineShowFlags.VisualizeMeshDistanceFields;
		EditorViewportClient->EngineShowFlags.SetVisualizeMeshDistanceFields(bToggleOn);
		SceneViewport->Invalidate();

		if (bToggleOn && IsShowRayTracingFallbackChecked())
		{
			ToggleShowRayTracingFallback();
		}
	}
}

bool SStaticMeshEditorViewport::IsShowDistanceFieldChecked() const
{
	return EditorViewportClient ? EditorViewportClient->EngineShowFlags.VisualizeMeshDistanceFields : false;
}

bool SStaticMeshEditorViewport::IsShowDistanceFieldVisible() const
{
	return true;
}

void SStaticMeshEditorViewport::ToggleShowRayTracingFallback()
{
	if (EditorViewportClient)
	{
		if (EditorViewportClient->EngineShowFlags.RayTracingDebug)
		{
			EditorViewportClient->EngineShowFlags.SetRayTracingDebug(false);
			EditorViewportClient->CurrentRayTracingDebugVisualizationMode = NAME_None;
			SetFloorAndEnvironmentVisibility(true);
		}
		else
		{
			EditorViewportClient->EngineShowFlags.SetRayTracingDebug(true);
			EditorViewportClient->CurrentRayTracingDebugVisualizationMode = "Barycentrics";
			SetFloorAndEnvironmentVisibility(false);

			if (IsShowNaniteFallbackChecked())
			{
				ToggleShowNaniteFallback();
			}

			if (IsShowDistanceFieldChecked())
			{
				ToggleShowDistanceField();
			}
		}
		SceneViewport->Invalidate();
	}
}

bool SStaticMeshEditorViewport::IsShowRayTracingFallbackChecked() const
{
	return EditorViewportClient ? EditorViewportClient->EngineShowFlags.RayTracingDebug : false;
}

bool SStaticMeshEditorViewport::IsShowRayTracingFallbackVisible() const
{
	return true;
}

void SStaticMeshEditorViewport::SetFloorAndEnvironmentVisibility(bool bVisible)
{
	if (IsInViewModeVertexColorChecked() || IsShowRayTracingFallbackChecked())
	{
		bVisible = false;
	}

	if (EditorViewportClient)
	{
		EditorViewportClient->SetFloorAndEnvironmentVisibility(bVisible);
	}
}

void SStaticMeshEditorViewport::SetAspectRatioState(EStaticMeshEditorAspectRatio AspectRatioIn)
{
	if (CurrentAspectRatioMode != AspectRatioIn)
	{
		CurrentAspectRatioMode = AspectRatioIn;
		switch (CurrentAspectRatioMode)
		{
		case EStaticMeshEditorAspectRatio::EFree:
			UnsetAspectRatio();
			break;
		case EStaticMeshEditorAspectRatio::EPlatform:
			SetUsePlatformAspectRatio();
			break;
		case EStaticMeshEditorAspectRatio::E6_13:
			SetAspectRatio(6.0f / 13.0f);
			break;
		case EStaticMeshEditorAspectRatio::E13_6:
			SetAspectRatio(13.0f / 6.0f);
			break;
		case EStaticMeshEditorAspectRatio::E1_1:
			SetAspectRatio(1.0f);
			break;
		case EStaticMeshEditorAspectRatio::E4_3:
			SetAspectRatio(4.0f / 3.0f);
			break;
		case EStaticMeshEditorAspectRatio::E16_10:
			SetAspectRatio(16.0f / 10.0f);
			break;
		case EStaticMeshEditorAspectRatio::E16_9:
			SetAspectRatio(16.0f / 9.0f);
			break;
		default:
			ensure(false);
		}
	}
}

bool SStaticMeshEditorViewport::IsAspectRatio(EStaticMeshEditorAspectRatio TestAspectRatio) const
{
	return CurrentAspectRatioMode == TestAspectRatio;
}

void SStaticMeshEditorViewport::UpdatePreviewSocketMeshes()
{
	UStaticMesh* const PreviewStaticMesh = PreviewMeshComponent ? ToRawPtr(PreviewMeshComponent->GetStaticMesh()) : nullptr;

	if( PreviewStaticMesh )
	{
		const int32 SocketedComponentCount = SocketPreviewMeshComponents.Num();
		const int32 SocketCount = PreviewStaticMesh->Sockets.Num();

		const int32 IterationCount = FMath::Max(SocketedComponentCount, SocketCount);
		for(int32 i = 0; i < IterationCount; ++i)
		{
			if(i >= SocketCount)
			{
				// Handle removing an old component
				UStaticMeshComponent* SocketPreviewMeshComponent = SocketPreviewMeshComponents[i];
				PreviewScene->RemoveComponent(SocketPreviewMeshComponent);
				SocketPreviewMeshComponents.RemoveAt(i, SocketedComponentCount - i);
				break;
			}
			else if(UStaticMeshSocket* Socket = PreviewStaticMesh->Sockets[i])
			{
				UStaticMeshComponent* SocketPreviewMeshComponent = NULL;

				// Handle adding a new component
				if(i >= SocketedComponentCount)
				{
					SocketPreviewMeshComponent = NewObject<UStaticMeshComponent>();
					PreviewScene->AddComponent(SocketPreviewMeshComponent, FTransform::Identity);
					SocketPreviewMeshComponents.Add(SocketPreviewMeshComponent);
					SocketPreviewMeshComponent->AttachToComponent(PreviewMeshComponent, FAttachmentTransformRules::SnapToTargetIncludingScale, Socket->SocketName);
				}
				else
				{
					SocketPreviewMeshComponent = SocketPreviewMeshComponents[i];

					// In case of a socket rename, ensure our preview component is still snapping to the proper socket
					if (!SocketPreviewMeshComponent->GetAttachSocketName().IsEqual(Socket->SocketName))
					{
						SocketPreviewMeshComponent->AttachToComponent(PreviewMeshComponent, FAttachmentTransformRules::SnapToTargetIncludingScale, Socket->SocketName);
					}

					// Force component to world update to take into account the new socket position.
					SocketPreviewMeshComponent->UpdateComponentToWorld();
				}

				SocketPreviewMeshComponent->SetStaticMesh(Socket->PreviewStaticMesh);
			}
		}
	}
}

void SStaticMeshEditorViewport::SetPreviewMesh(UStaticMesh* InStaticMesh)
{
	// Set the new preview static mesh.
	FComponentReregisterContext ReregisterContext( PreviewMeshComponent );
	PreviewMeshComponent->SetStaticMesh(InStaticMesh);

	FTransform Transform = FTransform::Identity;
	PreviewScene->AddComponent( PreviewMeshComponent, Transform );

	EditorViewportClient->SetPreviewMesh(InStaticMesh, PreviewMeshComponent);
}

void SStaticMeshEditorViewport::UpdatePreviewMesh(UStaticMesh* InStaticMesh, bool bResetCamera/*= true*/)
{
	{
		const int32 SocketedComponentCount = SocketPreviewMeshComponents.Num();
		for(int32 i = 0; i < SocketedComponentCount; ++i)
		{
			UStaticMeshComponent* SocketPreviewMeshComponent = SocketPreviewMeshComponents[i];
			if( SocketPreviewMeshComponent )
			{
				PreviewScene->RemoveComponent(SocketPreviewMeshComponent);
			}
		}
		SocketPreviewMeshComponents.Empty();
	}

	if (PreviewMeshComponent)
	{
		PreviewScene->RemoveComponent(PreviewMeshComponent);
		PreviewMeshComponent = NULL;
	}

	PreviewMeshComponent = NewObject<UStaticMeshComponent>();
	ERHIFeatureLevel::Type FeatureLevel = GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel();
	if ( FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		PreviewMeshComponent->SetMobility(EComponentMobility::Static);
	}
	PreviewMeshComponent->SetStaticMesh(InStaticMesh);

	PreviewScene->AddComponent(PreviewMeshComponent,FTransform::Identity);

	const int32 SocketCount = InStaticMesh->Sockets.Num();
	SocketPreviewMeshComponents.Reserve(SocketCount);
	for(int32 i = 0; i < SocketCount; ++i)
	{
		UStaticMeshSocket* Socket = InStaticMesh->Sockets[i];

		UStaticMeshComponent* SocketPreviewMeshComponent = NULL;
		if( Socket && Socket->PreviewStaticMesh )
		{
			SocketPreviewMeshComponent = NewObject<UStaticMeshComponent>();
			SocketPreviewMeshComponent->SetStaticMesh(Socket->PreviewStaticMesh);
			SocketPreviewMeshComponent->AttachToComponent(PreviewMeshComponent, FAttachmentTransformRules::SnapToTargetIncludingScale, Socket->SocketName);
			SocketPreviewMeshComponents.Add(SocketPreviewMeshComponent);
			PreviewScene->AddComponent(SocketPreviewMeshComponent, FTransform::Identity);
		}
	}

	EditorViewportClient->SetPreviewMesh(InStaticMesh, PreviewMeshComponent, bResetCamera);

	if (EditorViewportClient->EngineShowFlags.PhysicalMaterialMasks)
	{
		//Reapply the physical material masks mode on the newly set static mesh.
		SetViewModePhysicalMaterialMasksImplementation(true);
	}
	else if (EditorViewportClient->EngineShowFlags.VertexColors)
	{
		//Reapply the vertex color mode on the newly set static mesh.
		SetViewModeVertexColorImplementation(true);
	}

	PreviewMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &SStaticMeshEditorViewport::PreviewComponentSelectionOverride);
	PreviewMeshComponent->PushSelectionToProxy();
}

bool SStaticMeshEditorViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible();
}

UStaticMeshComponent* SStaticMeshEditorViewport::GetStaticMeshComponent() const
{
	return PreviewMeshComponent;
}

void SStaticMeshEditorViewport::SetViewModeWireframe()
{
	if(CurrentViewMode != VMI_Wireframe)
	{
		CurrentViewMode = VMI_Wireframe;
	}
	else
	{
		CurrentViewMode = VMI_Lit;
	}
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("CurrentViewMode"), FString::Printf(TEXT("%d"), static_cast<int32>(CurrentViewMode)));
	}
	EditorViewportClient->SetViewMode(CurrentViewMode);
	SceneViewport->Invalidate();

}

bool SStaticMeshEditorViewport::IsInViewModeWireframeChecked() const
{
	return CurrentViewMode == VMI_Wireframe;
}

void SStaticMeshEditorViewport::SetViewModeVertexColor()
{
	SetViewModeVertexColorImplementation(!EditorViewportClient->EngineShowFlags.VertexColors);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), FAnalyticsEventAttribute(TEXT("VertexColors"), static_cast<int>(EditorViewportClient->EngineShowFlags.VertexColors)));
	}
}

void SStaticMeshEditorViewport::SetViewModeVertexColorImplementation(bool bValue)
{
	SetViewModeVertexColorSubImplementation(bValue);

	// Disable physical material masks, if enabling vertex color.
	if (bValue)
	{
		SetViewModePhysicalMaterialMasksSubImplementation(false);
	}

	PreviewMeshComponent->MarkRenderStateDirty();
	SceneViewport->Invalidate();
}

void SStaticMeshEditorViewport::SetViewModeVertexColorSubImplementation(bool bValue)
{
	EditorViewportClient->EngineShowFlags.SetVertexColors(bValue);
	EditorViewportClient->EngineShowFlags.SetLighting(!bValue);
	EditorViewportClient->EngineShowFlags.SetIndirectLightingCache(!bValue);
	EditorViewportClient->EngineShowFlags.SetPostProcessing(!bValue);
	SetFloorAndEnvironmentVisibility(!bValue);
	PreviewMeshComponent->bDisplayVertexColors = bValue;
}

bool SStaticMeshEditorViewport::IsInViewModeVertexColorChecked() const
{
	return EditorViewportClient->EngineShowFlags.VertexColors;
}

void SStaticMeshEditorViewport::SetViewModePhysicalMaterialMasks()
{
	SetViewModePhysicalMaterialMasksImplementation(!EditorViewportClient->EngineShowFlags.PhysicalMaterialMasks);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), FAnalyticsEventAttribute(TEXT("PhysicalMaterialMasks"), static_cast<int>(EditorViewportClient->EngineShowFlags.PhysicalMaterialMasks)));
	}
}

void SStaticMeshEditorViewport::SetViewModePhysicalMaterialMasksImplementation(bool bValue)
{
	SetViewModePhysicalMaterialMasksSubImplementation(bValue);

	// Disable vertex color, if enabling physical material masks.
	if (bValue)
	{
		SetViewModeVertexColorSubImplementation(false);
	}

	PreviewMeshComponent->MarkRenderStateDirty();
	SceneViewport->Invalidate();
}

void SStaticMeshEditorViewport::SetViewModePhysicalMaterialMasksSubImplementation(bool bValue)
{
	EditorViewportClient->EngineShowFlags.SetPhysicalMaterialMasks(bValue);
	PreviewMeshComponent->bDisplayPhysicalMaterialMasks = bValue;
}

bool SStaticMeshEditorViewport::IsInViewModePhysicalMaterialMasksChecked() const
{
	return EditorViewportClient->EngineShowFlags.PhysicalMaterialMasks;
}


void SStaticMeshEditorViewport::ForceLODLevel(int32 InForcedLOD)
{
	LODSelection = InForcedLOD;

	FStaticMeshRayTracingProxy* RayTracingProxy = nullptr;
	if (PreviewMeshComponent->GetStaticMesh() && PreviewMeshComponent->GetStaticMesh()->GetRenderData())
	{
		RayTracingProxy = PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy;
	}

	if (IsShowRayTracingFallbackChecked())
	{
		PreviewMeshComponent->ForcedLodModel = 0;

		if (RayTracingProxy)
		{
			RayTracingProxy->PreviewLODLevel = LODSelection - 1;
		}
	}
	else
	{
		PreviewMeshComponent->ForcedLodModel = LODSelection;

		if (RayTracingProxy)
		{
			RayTracingProxy->PreviewLODLevel = INDEX_NONE;
		}
	}
	
	{FComponentReregisterContext ReregisterContext(PreviewMeshComponent);}
	SceneViewport->Invalidate();
}

int32 SStaticMeshEditorViewport::GetCurrentLOD() const
{
	if (PreviewMeshComponent)
	{
		if (IsShowRayTracingFallbackChecked())
		{
			if (PreviewMeshComponent->GetStaticMesh()
				&& PreviewMeshComponent->GetStaticMesh()->GetRenderData()
				&& PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy)
			{
				return PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy->PreviewLODLevel;
			}
		}
		else
		{
			return PreviewMeshComponent->ForcedLodModel - 1;
		}
	}
	return INDEX_NONE;
}

bool SStaticMeshEditorViewport::IsLODSelected(int32 InLODSelection) const
{
	if (PreviewMeshComponent)
	{
		if (IsShowRayTracingFallbackChecked())
		{
			if (PreviewMeshComponent->GetStaticMesh()
				&& PreviewMeshComponent->GetStaticMesh()->GetRenderData()
				&& PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy)
			{
				return InLODSelection == PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy->PreviewLODLevel;
			}
		}
		else
		{
			return PreviewMeshComponent->ForcedLodModel - 1 == InLODSelection;
		}
	}
	return false;
}

void SStaticMeshEditorViewport::SetLODLevel(int32 InLODSelection)
{
	if (PreviewMeshComponent)
	{
		LODSelection = InLODSelection;

		FStaticMeshRayTracingProxy* RayTracingProxy = nullptr;
		if (PreviewMeshComponent->GetStaticMesh() && PreviewMeshComponent->GetStaticMesh()->GetRenderData())
		{
			RayTracingProxy = PreviewMeshComponent->GetStaticMesh()->GetRenderData()->RayTracingProxy;
		}

		if (IsShowRayTracingFallbackChecked())
		{
			PreviewMeshComponent->bOverrideMinLOD = false;
			PreviewMeshComponent->SetForcedLodModel(0);

			if (RayTracingProxy)
			{
				RayTracingProxy->PreviewLODLevel = LODSelection;
				PreviewMeshComponent->MarkRenderStateDirty();
			}
		}
		else
		{
			PreviewMeshComponent->bOverrideMinLOD = (LODSelection >= 0);
			PreviewMeshComponent->SetForcedLodModel(LODSelection + 1);

			if (RayTracingProxy)
			{
				RayTracingProxy->PreviewLODLevel = INDEX_NONE;
				PreviewMeshComponent->MarkRenderStateDirty();
			}
		}

		//PopulateUVChoices();
		StaticMeshEditorPtr.Pin()->BroadcastOnSelectedLODChanged();
		RefreshViewport();
	}
}

void SStaticMeshEditorViewport::FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands)
{
	Commands.Add(FStaticMeshViewportLODCommands::Get().LODAuto);
	Commands.Add(FStaticMeshViewportLODCommands::Get().LOD0);
}

void SStaticMeshEditorViewport::OnLODModelChanged()
{
	if (PreviewMeshComponent && LODSelection != PreviewMeshComponent->ForcedLodModel)
	{
		//PopulateUVChoices();
	}
}

void SStaticMeshEditorViewport::SetCurrentScreenPercentage(float ScreenPercentage)
{
	CurrentScreenPercentage = ScreenPercentage;

	if (LODVisualizer)
	{
		LODVisualizer->SetScreenPercentage(CurrentScreenPercentage);
	}
}

float SStaticMeshEditorViewport::GetScreenPercentageForLOD(int LODIndex)
{
	float ScreenSize = 0.0;
	FName PlatformGroupName = NAME_None;

	if (LODIndex < 0)
	{
		return 1.0;
	}

	if (LODIndex >= MAX_STATIC_MESH_LODS)
	{
		return 0.0;
	}

	if (PreviewMeshComponent)
	{
		UStaticMesh* Mesh = PreviewMeshComponent->GetStaticMesh();
		if (Mesh)
		{
			if (Mesh->GetAutoComputeLODScreenSize() && Mesh->GetRenderData())
			{
				ScreenSize = Mesh->GetRenderData()->ScreenSize[LODIndex].Default;
				const float* PlatformScreenSize = Mesh->GetRenderData()->ScreenSize[LODIndex].PerPlatform.Find(PlatformGroupName);
				if (PlatformScreenSize != nullptr)
				{
					ScreenSize = *PlatformScreenSize;
				}

			}
			else if (Mesh->IsSourceModelValid(LODIndex))
			{
				ScreenSize = Mesh->GetSourceModel(LODIndex).ScreenSize.Default;
				const float* PlatformScreenSize = Mesh->GetSourceModel(LODIndex).ScreenSize.PerPlatform.Find(PlatformGroupName);
				if (PlatformScreenSize != nullptr)
				{
					ScreenSize = *PlatformScreenSize;
				}
			}
		}
	}
	return ScreenSize;
}

int32 SStaticMeshEditorViewport::GetCurrentMinimumLOD() const
{
	if (PreviewMeshComponent)
	{
		if (IsShowRayTracingFallbackChecked())
		{
			return 0;
		}
		else if(PreviewMeshComponent->GetStaticMesh())
		{
			return PreviewMeshComponent->GetStaticMesh()->GetMinLODIdx();
		}
	}

	return 0;
}


int32 SStaticMeshEditorViewport::GetLODCount() const
{
	if (PreviewMeshComponent && PreviewMeshComponent->GetStaticMesh())
	{
		if (IsShowRayTracingFallbackChecked())
		{
			if (StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->RayTracingProxy)
			{
				return StaticMesh->GetRenderData()->RayTracingProxy->LODs.Num();
			}
		}
		else
		{
			return StaticMesh->GetNumLODs();
		}
	}
	return 0;
}

TSet< int32 >& SStaticMeshEditorViewport::GetSelectedEdges()
{
	return EditorViewportClient->GetSelectedEdges();
}

FStaticMeshEditorViewportClient& SStaticMeshEditorViewport::GetViewportClient()
{
	return *EditorViewportClient;
}


TSharedRef<FEditorViewportClient> SStaticMeshEditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable( new FStaticMeshEditorViewportClient(StaticMeshEditorPtr, SharedThis(this), PreviewScene.ToSharedRef(), StaticMesh, NULL) );

	EditorViewportClient->bSetListenerPosition = false;

	EditorViewportClient->SetRealtime( true );
	EditorViewportClient->VisibilityDelegate.BindSP( this, &SStaticMeshEditorViewport::IsVisible );

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<IPreviewProfileController> SStaticMeshEditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

EVisibility SStaticMeshEditorViewport::OnGetViewportContentVisibility() const
{
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SStaticMeshEditorViewport::BindCommands()
{
	SAssetEditorViewport::BindCommands();

	const FStaticMeshEditorCommands& Commands = FStaticMeshEditorCommands::Get();

	TSharedRef<FStaticMeshEditorViewportClient> EditorViewportClientRef = EditorViewportClient.ToSharedRef();

	CommandList->MapAction(
		Commands.SetShowNaniteFallback,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::ToggleShowNaniteFallback),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsShowNaniteFallbackChecked),
		FIsActionButtonVisible::CreateSP(this, &SStaticMeshEditorViewport::IsShowNaniteFallbackVisible));

	CommandList->MapAction(
		Commands.SetShowDistanceField,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::ToggleShowDistanceField),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsShowDistanceFieldChecked),
		FIsActionButtonVisible::CreateSP(this, &SStaticMeshEditorViewport::IsShowDistanceFieldVisible));

	CommandList->MapAction(
		Commands.SetShowRayTracingFallback,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::ToggleShowRayTracingFallback),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsShowRayTracingFallbackChecked),
		FIsActionButtonVisible::CreateSP(this, &SStaticMeshEditorViewport::IsShowRayTracingFallbackVisible));

	CommandList->MapAction(
		Commands.SetShowWireframe,
		FExecuteAction::CreateSP( this, &SStaticMeshEditorViewport::SetViewModeWireframe ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SStaticMeshEditorViewport::IsInViewModeWireframeChecked ) );

	CommandList->MapAction(
		Commands.SetShowVertexColor,
		FExecuteAction::CreateSP( this, &SStaticMeshEditorViewport::SetViewModeVertexColor ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SStaticMeshEditorViewport::IsInViewModeVertexColorChecked ) );

	CommandList->MapAction(
		Commands.SetShowPhysicalMaterialMasks,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetViewModePhysicalMaterialMasks),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsInViewModePhysicalMaterialMasksChecked));

	CommandList->MapAction(
		Commands.SetDrawUVs,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleDrawUVOverlay ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsDrawUVOverlayChecked ) );

	CommandList->MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::SetShowGrid ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsSetShowGridChecked ) );

	CommandList->MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowBounds ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsSetShowBoundsChecked ) );

	CommandList->MapAction(
		Commands.SetShowSimpleCollision,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowSimpleCollision ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowSimpleCollisionChecked ) );

	CommandList->MapAction(
		Commands.SetShowComplexCollision,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowComplexCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowComplexCollisionChecked));

	CommandList->MapAction(
		Commands.SetShowSockets,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowSockets ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowSocketsChecked ) );

	// Menu
	CommandList->MapAction(
		Commands.SetShowNormals,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowNormals ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowNormalsChecked ) );

	CommandList->MapAction(
		Commands.SetShowTangents,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowTangents ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowTangentsChecked ) );

	CommandList->MapAction(
		Commands.SetShowBinormals,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowBinormals ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowBinormalsChecked ) );

	CommandList->MapAction(
		Commands.SetShowPivot,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleShowPivot ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsShowPivotChecked ) );

	CommandList->MapAction(
		Commands.SetDrawAdditionalData,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleDrawAdditionalData ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsDrawAdditionalDataChecked ) );

	CommandList->MapAction(
		Commands.SetShowVertices,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FStaticMeshEditorViewportClient::ToggleDrawVertices ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FStaticMeshEditorViewportClient::IsDrawVerticesChecked ) );

	// LOD
	StaticMeshEditorPtr.Pin()->RegisterOnSelectedLODChanged(FOnSelectedLODChanged::CreateSP(this, &SStaticMeshEditorViewport::OnLODModelChanged), false);
	//Bind LOD preview menu commands

	const FStaticMeshViewportLODCommands& ViewportLODMenuCommands = FStaticMeshViewportLODCommands::Get();
	
	//LOD Auto
	CommandList->MapAction(
		ViewportLODMenuCommands.LODAuto,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetLODLevel, -1),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsLODSelected, -1));

	// LOD 0
	CommandList->MapAction(
		ViewportLODMenuCommands.LOD0,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetLODLevel, 0),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsLODSelected, 0));
	// all other LODs will be added dynamically


	// Aspect Ratio
	CommandList->MapAction(
		Commands.SetAspectRatio_Free,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::EFree),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::EFree));

	CommandList->MapAction(
		Commands.SetAspectRatio_Platform,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::EPlatform),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::EPlatform));

	CommandList->MapAction(
		Commands.SetAspectRatio_6_13,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E6_13),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E6_13));

	CommandList->MapAction(
		Commands.SetAspectRatio_13_6,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E13_6),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E13_6));

	CommandList->MapAction(
		Commands.SetAspectRatio_1_1,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E1_1),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E1_1));

	CommandList->MapAction(
		Commands.SetAspectRatio_4_3,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E4_3),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E4_3));

	CommandList->MapAction(
		Commands.SetAspectRatio_16_10,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E16_10),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E16_10));

	CommandList->MapAction(
		Commands.SetAspectRatio_16_9,
		FExecuteAction::CreateSP(this, &SStaticMeshEditorViewport::SetAspectRatioState, EStaticMeshEditorAspectRatio::E16_9),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStaticMeshEditorViewport::IsAspectRatio, EStaticMeshEditorAspectRatio::E16_9));
}

void SStaticMeshEditorViewport::OnFocusViewportToSelection()
{
	// If we have selected sockets, focus on them
	const TArray<UStaticMeshSocket*> SelectedSockets = StaticMeshEditorPtr.Pin()->GetSelectedSockets();

	if (SelectedSockets.Num() && PreviewMeshComponent)
	{
		FTransform SocketTransform;
		SelectedSockets[0]->GetSocketTransform(SocketTransform, PreviewMeshComponent);

		const FVector Origin = SocketTransform.GetLocation();

		FBox Box(Origin, Origin);

		for (const UStaticMeshSocket* Socket : SelectedSockets)
		{
			Socket->GetSocketTransform(SocketTransform, PreviewMeshComponent);

			Box.Max.X = FMath::Max(SocketTransform.GetLocation().X, Box.Max.X);
			Box.Max.Y = FMath::Max(SocketTransform.GetLocation().Y, Box.Max.Y);
			Box.Max.Z = FMath::Max(SocketTransform.GetLocation().Z, Box.Max.Z);

			Box.Min.X = FMath::Min(SocketTransform.GetLocation().X, Box.Min.X);
			Box.Min.Y = FMath::Min(SocketTransform.GetLocation().Y, Box.Min.Y);
			Box.Min.Z = FMath::Min(SocketTransform.GetLocation().Z, Box.Min.Z);
		}

		Box.Max += FVector(30.f);
		Box.Min -= FVector(30.f);

		EditorViewportClient->FocusViewportOnBox(Box);
		return;
	}

	// If we have selected primitives, focus on them
	FBox Box(ForceInit);
	const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->CalcSelectedPrimsAABB(Box);
	if (bSelectedPrim)
	{
		EditorViewportClient->FocusViewportOnBox(Box);
		return;
	}

	// Fallback to focusing on the mesh, if nothing else
	if (PreviewMeshComponent)
	{
		EditorViewportClient->FocusViewportOnBox(PreviewMeshComponent->Bounds.GetBox());
		return;
	}
}

TSharedPtr<SWidget> SStaticMeshEditorViewport::BuildViewportToolbar()
{
	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	const FName ViewportToolbarName = "StaticMeshEditor.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
			LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));

			// Add the "View Modes" sub menu.
			{
				const FName ViewModesMenuName = "StaticMeshEditor.ViewportToolbar.ViewModes";

				// Stay backward-compatible with the old viewport toolbar.
				{
					const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
					{
						UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
					}

					// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
					UToolMenus::Get()->RegisterMenu(ViewModesMenuName, ParentSubmenuName);
				}

				UE::StaticMeshEditor::ExtendViewModesSubmenu(ViewModesMenuName);

				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			RightSection.AddEntry(UE::StaticMeshEditor::CreateShowSubmenu());

			RightSection.AddDynamicEntry("StaticMeshEditorAspectRatioControls", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (StaticMeshEditorViewportLocals::CVarEnableStaticMeshEditorAspectRatioControl.GetValueOnGameThread())
				{
					InSection.AddEntry(UE::StaticMeshEditor::CreateAspectRatioSubmenu());
				}
			}));										

			RightSection.AddEntry(UE::StaticMeshEditor::CreateLODSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());

			// Add Preview Scene Submenu
			{
				const FName PreviewSceneMenuName = "StaticMeshEditor.ViewportToolbar.AssetViewerProfile";
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
				UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(PreviewSceneMenuName);
				UE::UnrealEd::ExtendPreviewSceneSettingsWithTabEntry(PreviewSceneMenuName);
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(PreviewScene->GetCommandList());
		ViewportToolbarContext.AppendCommandList(GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));

			ContextObject->bShowCoordinateSystemControls = false;

			ContextObject->AssetEditorToolkit = StaticMeshEditorPtr;
			ContextObject->PreviewSettingsTabId = FStaticMeshEditor::PreviewSceneSettingsTabId;

			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

#undef LOCTEXT_NAMESPACE
