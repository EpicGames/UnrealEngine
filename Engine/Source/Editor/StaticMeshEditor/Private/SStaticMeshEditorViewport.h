// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EditorViewportClient.h"
#include "StaticMeshEditorViewportClient.h"
#include "AdvancedPreviewScene.h"
#include "IPreviewLODController.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SAssetEditorViewport.h"
#include "Styling/StyleColors.h"

class IStaticMeshEditor;
class SVerticalBox;
class UStaticMesh;
class UStaticMeshComponent;
class SRichTextBlock;
class SStaticMeshLODVisualizerWidget;

enum class EStaticMeshEditorAspectRatio : int32
{
	EFree = 0,
	EPlatform = 1,

	E6_13 = 100,
	E13_6,
	E1_1,
	E4_3,
	E16_10,
	E16_9,
};

/**
 * StaticMesh Editor Preview viewport widget
 */
class SStaticMeshEditorViewport : public SAssetEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider, public IPreviewLODController
{
public:
	SLATE_BEGIN_ARGS( SStaticMeshEditorViewport ){}
		SLATE_ARGUMENT(TWeakPtr<IStaticMeshEditor>, StaticMeshEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	SStaticMeshEditorViewport();
	virtual ~SStaticMeshEditorViewport() override;
	
	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SStaticMeshEditorViewport");
	}
	// End of FGCObject interface

	/** Constructs, destroys, and updates preview mesh components based on the preview static mesh's sockets. */
	void UpdatePreviewSocketMeshes();

	void RefreshViewport();
	
	/** Component for the preview static mesh. */
	TObjectPtr<UStaticMeshComponent> PreviewMeshComponent;

	/** Component for the preview static mesh. */
	TArray<TObjectPtr<UStaticMeshComponent>> SocketPreviewMeshComponents;

	/** 
	 *	Forces a specific LOD level onto the static mesh component.
	 *
	 *	@param	InForcedLOD			The desired LOD to be forced to.
	 */
	void ForceLODLevel(int32 InForcedLOD);

	
	// IPreviewLODController interface
	/**
	 *
	 * Query LOD level of static mesh component 
	 *
	 */
	virtual int32 GetCurrentLOD() const override;
	virtual int32 GetLODCount() const override;
	virtual bool IsLODSelected(int32 LODIndex) const override;
	virtual void SetLODLevel(int32 LODIndex) override;
	virtual void FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands) override;
	virtual int32 GetAutoLODStartingIndex() const override { return 1; }
	// ~IPreviewLODController

	void OnLODModelChanged();

	void SetCurrentScreenPercentage(float ScreenPercentage);
	float GetScreenPercentageForLOD(int LODIndex);
	int32 GetCurrentMinimumLOD() const;

	/** Retrieves the static mesh component. */
	UStaticMeshComponent* GetStaticMeshComponent() const;

	/**
	 *	Sets up the static mesh that the Static Mesh editor is viewing.
	 *
	 *	@param	InStaticMesh		The static mesh being viewed in the editor.
	 */
	void SetPreviewMesh(UStaticMesh* InStaticMesh);

	/**
	 *	Updates the preview mesh and other viewport specfic settings that go with it.
	 *
	 *	@param	InStaticMesh		The static mesh being viewed in the editor.
	 */
	void UpdatePreviewMesh(UStaticMesh* InStaticMesh, bool bResetCamera=true);

	/** Retrieves the selected edge set. */
	TSet< int32 >& GetSelectedEdges();

	/** @return The editor viewport client */
	class FStaticMeshEditorViewportClient& GetViewportClient();

	/** Set the parent tab of the viewport for determining visibility */
	void SetParentTab( TSharedRef<SDockTab> InParentTab ) { ParentTab = InParentTab; }

	/** Struct defining the text and its style of each item in the overlay widget */
	struct FOverlayTextItem
	{
		explicit FOverlayTextItem(const FText& InText, bool bInIsWarning = false, bool bInIsCustomFormat=false)
			: Text(InText), bIsWarning(bInIsWarning), bIsCustomFormat(bInIsCustomFormat)
		{}

		FText Text;
		bool bIsWarning;
		bool bIsCustomFormat;
	};

	/** Specifies an array of text items which will be added to the viewport overlay */
	void PopulateOverlayText( const TArrayView<FOverlayTextItem> TextItems );

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	/** Returns the preview scene being renderd in the viewport */
	TSharedRef<FAdvancedPreviewScene> GetPreviewScene() { return PreviewScene.ToSharedRef(); }

	TSharedPtr<SOverlay> GetViewportOverlay() { return ViewportOverlay; }

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;

public:

	/** Callback for toggling the vertex color show flag. */
	void SetViewModeVertexColor();

	/** Callback for checking the vertex color show flag. */
	bool IsInViewModeVertexColorChecked() const;

	/** Callback for toggling the wireframe mode. */
	void SetViewModeWireframe();

	/** Callback for checking the wireframe flag. */
	bool IsInViewModeWireframeChecked() const;

private:

	/** Determines the visibility of the viewport. */
	virtual bool IsVisible() const override;

	/** Implementation of the SetViewModeVertexColor, used to apply the vertex color show flag. */
	void SetViewModeVertexColorImplementation(bool bValue);

	/** Applies the vertex color show flag only. */
	void SetViewModeVertexColorSubImplementation(bool bValue);

	/** Callback for toggling the physical material mask show flag. */
	void SetViewModePhysicalMaterialMasks();

	/** Implementation of the SetViewModeVertexColor, used to apply the physical material mask show flag. */
	void SetViewModePhysicalMaterialMasksImplementation(bool bValue);

	/** Applies the physical material mask show flag only. */
	void SetViewModePhysicalMaterialMasksSubImplementation(bool bValue);

	/** Callback for checking the physical material mask show flag. */
	bool IsInViewModePhysicalMaterialMasksChecked() const;

	/** Callback for updating preview socket meshes if the static mesh or socket has been modified. */
	void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Override for preview component selection to inform the editor we consider it selected. */
	bool PreviewComponentSelectionOverride(const UPrimitiveComponent* InComponent) const;

	void ToggleShowNaniteFallback();

	bool IsShowNaniteFallbackChecked() const;

	bool IsShowNaniteFallbackVisible() const;

	void ToggleShowDistanceField();

	bool IsShowDistanceFieldChecked() const;

	bool IsShowDistanceFieldVisible() const;

	void ToggleShowRayTracingFallback();

	bool IsShowRayTracingFallbackChecked() const;

	bool IsShowRayTracingFallbackVisible() const;

	void SetFloorAndEnvironmentVisibility(bool bVisible);

	void SetAspectRatioState(EStaticMeshEditorAspectRatio TestAspectRatio);

	bool IsAspectRatio(EStaticMeshEditorAspectRatio TestAspectRatio) const;

private:
	
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	/** Pointer back to the StaticMesh editor tool that owns us */
	TWeakPtr<IStaticMeshEditor> StaticMeshEditorPtr;

	/** The scene for this viewport. */
	TSharedPtr<FAdvancedPreviewScene> PreviewScene;

	/** Editor viewport client */
	TSharedPtr<class FStaticMeshEditorViewportClient> EditorViewportClient;

	/** Static mesh being edited */
	TObjectPtr<UStaticMesh> StaticMesh;

	/** The currently selected view mode. */
	EViewModeIndex CurrentViewMode;

	/** Pointer to the box into which the overlay text items are added */
	TSharedPtr<SRichTextBlock> OverlayText;

	/** Pointer to a LOD visualizer widget to display in the viewport */
	TSharedPtr<SStaticMeshLODVisualizerWidget> LODVisualizer;

	/** The current screen percentage occupied by the asset in the viewport */
	float CurrentScreenPercentage = 0.f;

	/** Current LOD Selection where 0 is Auto */
	int32 LODSelection;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	/** Current aspect ratio the viewport is set to.*/
	EStaticMeshEditorAspectRatio CurrentAspectRatioMode = EStaticMeshEditorAspectRatio::EPlatform;
};
