// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStaticMeshLODVisualizerWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h" // FDragDropEvent
#include "Styling/SlateTypes.h" // FWindowStyle
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h" // SVerticalBox
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "StaticMeshEditorStyle.h"


#define LOCTEXT_NAMESPACE "SStaticMeshLODVisualizerWidget"

namespace LODVisualizerWidgetLocals
{
	static FColor LODColors[8] = {
		FColor(141, 211, 199),
		FColor(255, 255, 179),
		FColor(190, 186, 218),
		FColor(251, 128, 114),
		FColor(128, 177, 211),
		FColor(253, 180, 98),
		FColor(179, 222, 105),
		FColor(252, 205, 229)
	};
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SStaticMeshLODVisualizerWidget::Construct(const FArguments& InArgs)
{
	SliderValue.BindLambda([this]() { return ScreenPercentage; });
	PercentAbove.BindLambda([this]() { return 1-ScreenPercentage; });
	PercentBelow.BindLambda([this]() { return ScreenPercentage; });

	NaniteOverlayVisibility.BindLambda([this]() { return (IsNaniteOverlayActive()) ? EVisibility::Visible : EVisibility::Collapsed; });

	PerLODData.SetNum(8);

	for (int32 LODIndex = 0; LODIndex < 8; ++LODIndex)
	{
		PerLODData[LODIndex].ActivationPercentage.BindLambda([this, LODIndex]() { return ComputeLODSectionFill(LODIndex); });
		PerLODData[LODIndex].Visibility.BindLambda([this, LODIndex]() { return NumLODs > LODIndex ? EVisibility::Visible : EVisibility::Collapsed; });
		PerLODData[LODIndex].Color.BindLambda([this, LODIndex]()
			{
				float ReductionFactor = ((ActiveLOD == LODIndex || ActiveLOD == -1) && !IsNaniteOverlayActive()) ? 1.0 : 0.5;
				return FSlateColor(ReductionFactor * LODVisualizerWidgetLocals::LODColors[LODIndex]);
			});
		PerLODData[LODIndex].MinimalLODVisibility.BindLambda([this, LODIndex]() { return (MinimumLOD > LODIndex) ? EVisibility::Visible : EVisibility::Hidden; });
		PerLODData[LODIndex].ToolTip.BindLambda([this, LODIndex]() {
			if (((ActiveLOD == LODIndex || ActiveLOD == -1) && !IsNaniteOverlayActive()))
			{
				if (MinimumLOD > LODIndex)
				{
					return LOCTEXT("MinimumLODTooltip", "Disabled on this Platform due to Minimum LOD restrictions");
				}
				else
				{
					return FText::Format(LOCTEXT("NormalLODTooltip", "LOD {0} Visible At {1}"), LODIndex, PerLODData[LODIndex].ScreenPercentage);
				}
			}
			return FText();
		});
		
	}

	FNumberFormattingOptions Options;
	Options.MaximumFractionalDigits = 3;
	Options.MinimumFractionalDigits = 3;


	ChildSlot
		[
				SNew(SExpandableArea)
				.MinWidth(80)
				.BodyBorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
				.HeaderContent()
				[
					SNew(STextBlock)
					.Text( LOCTEXT( "WidgetLabel", "Show LOD Meter" ))
				]

				.BodyContent()
				[
					SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					.VAlign(EVerticalAlignment::VAlign_Fill)
					[
						SNew(SVerticalBox)
						 + SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[						
							SNew(STextBlock)
							.Font(FCoreStyle::Get().GetFontStyle("Font.Bold"))
							.Text_Lambda([this]() {
								return FText::Format(LOCTEXT("PlatformLabel", "Platform: {0}"), FText::FromString(CurrentPlatform));
							})
						]

						 + SVerticalBox::Slot()
						.FillHeight(1.0)
						.HAlign(HAlign_Center)
						[

							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 8)
							[
								SNew(SVerticalBox)							
									+ SVerticalBox::Slot()
									.FillHeight(PercentAbove)							
									[
										SNew(SBorder)
										.BorderImage(FStaticMeshEditorStyle::Get().GetBrush("StaticMeshEditor.LODMeter.NoBrush"))
									]
									+ SVerticalBox::Slot()
									.AutoHeight()
									.HAlign(HAlign_Right)
									[
										SNew(SBox)
										.WidthOverride(30)
										[
											SNew(STextBlock)
											.ColorAndOpacity(FSlateColor(FColor::White))
											.Text_Lambda([this, Options]()
											{
												return FText::Format(LOCTEXT("ScreenPercent_Label", "{0}"), FText::AsNumber(ScreenPercentage, &Options));
											})
										]
									]
									+ SVerticalBox::Slot()
									.FillHeight(PercentBelow)
									[
										SNew(SBorder)
										.BorderImage(FStaticMeshEditorStyle::Get().GetBrush("StaticMeshEditor.LODMeter.NoBrush"))
									]
							]

							 + SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SSlider)
								.Orientation(EOrientation::Orient_Vertical)
								.Locked(true)
								.Value(SliderValue)
								.Style(FStaticMeshEditorStyle::Get(), "StaticMeshEditor.LODMeter.SliderStyle")
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBox)
								.WidthOverride(50)
								[
									SNew(SOverlay)
										+ SOverlay::Slot()
										.Padding(0, 15)
									[
									   SAssignNew(LODSlotDisplay, SVerticalBox)
 									]


										 + SOverlay::Slot()
										.Padding(0, 15)
									[
										SNew(SHorizontalBox)
										.Visibility(NaniteOverlayVisibility)
										.ToolTipText(LOCTEXT("NaniteLODTooltip", "Nanite replaces traditional LODs on this Platform."))

										+ SHorizontalBox::Slot()
										.FillWidth(1)
										[
											SNew(SBorder)
											.Visibility(EVisibility::Hidden)
											.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
										]

										+ SHorizontalBox::Slot()										
										.FillWidth(5)
										[
											SNew(SBox)
											.WidthOverride(150)
											[
												SNew(SBorder)										
												.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
												.BorderBackgroundColor(FColor(200, 200, 200))
												.ForegroundColor(FSlateColor(FColor::Black))
												.VAlign(VAlign_Center)
												.HAlign(HAlign_Center)
												[
													SNew(STextBlock)
													.RenderTransform(FSlateRenderTransform(FQuat2f(FMath::DegreesToRadians(90)), FVector2f::Zero()))
													.RenderTransformPivot(FVector2d(.5,.5))																
													.Font(FCoreStyle::Get().GetFontStyle("Font.Large.Bold"))
													.Text(LOCTEXT("NaniteLabel", "Nanite LOD"))
													.Clipping(EWidgetClipping::Inherit)

												]
											]
										]
										+ SHorizontalBox::Slot()
										.FillWidth(1)
										[
											SNew(SBorder)
											.Visibility(EVisibility::Hidden)
											.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
										]

 									]

									+ SOverlay::Slot()
										.Padding(0, 15)
										[
											SNew(SVerticalBox)
												.Visibility(EVisibility::HitTestInvisible)
												+ SVerticalBox::Slot()
												.FillHeight(PercentAbove)
												[
													SNew(SBorder)
												]
												+ SVerticalBox::Slot()
												.AutoHeight()
												[
													SNew(SBorder)
													.Padding(0.0, 0.5, 0.0, 0.5)
													.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
													.BorderBackgroundColor(FSlateColor(FLinearColor(0,0,0)))
												]
												+ SVerticalBox::Slot()
												.FillHeight(PercentBelow)
												[
													SNew(SBorder)
												]
										]
								]
							]

						]
				
					]

				]
			]
	;
		
	for (int32 LODIndex = 0; LODIndex < 8; ++LODIndex)
	{
		LODSlotDisplay->AddSlot()
		.FillHeight(PerLODData[LODIndex].ActivationPercentage)
		[
			SNew(SBorder)
			.Visibility(PerLODData[LODIndex].Visibility)
			.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(PerLODData[LODIndex].Color)
			.ForegroundColor(FSlateColor(FColor::Black))
			.ToolTipText(PerLODData[LODIndex].ToolTip)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Padding(0)
			[
				SNew(SOverlay)
				.ToolTipText(PerLODData[LODIndex].ToolTip)
				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)				
				[
				SNew(SBorder)
				.Visibility(PerLODData[LODIndex].MinimalLODVisibility)
				.BorderImage(FStaticMeshEditorStyle::Get().GetBrush("StaticMeshEditor.LODMeter.DisabledBrush"))
				.BorderBackgroundColor(PerLODData[LODIndex].Color)
				.ToolTipText(PerLODData[LODIndex].ToolTip)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
			    .HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text( FText::Format( LOCTEXT("LOD_Label", "LOD {0}"), LODIndex ))
					.ToolTipText(PerLODData[LODIndex].ToolTip)
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SBox)
					.ToolTipText(PerLODData[LODIndex].ToolTip)
				]
			]
		];
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SStaticMeshLODVisualizerWidget::SetScreenPercentage(float NewScreenPercentageIn)
{
	ScreenPercentage = FMath::Clamp(NewScreenPercentageIn, 0.0, 1.0);
}

void SStaticMeshLODVisualizerWidget::SetNumLODs(int NumLodsIn)
{
	NumLODs = FMath::Clamp(NumLodsIn, 1, 8);
}

void SStaticMeshLODVisualizerWidget::SetLODPercentage(int32 LODIndex, float NewLODPercentageIn)
{
	if (LODIndex < 0 || LODIndex > 7)
	{
		return;
	}

	PerLODData[LODIndex].ScreenPercentage = NewLODPercentageIn;
}

void SStaticMeshLODVisualizerWidget::SetActiveLOD(int LODIndexIn)
{
	ActiveLOD = LODIndexIn;
}

float SStaticMeshLODVisualizerWidget::ComputeLODSectionFill(int32 LODIndex)
{
	float UpperValue = 1;
	float LowerValue = 0;

	if (LODIndex > 0)
	{
		UpperValue = PerLODData[LODIndex].ScreenPercentage;
	}

	if (LODIndex < 7)
	{
		LowerValue = PerLODData[LODIndex + 1].ScreenPercentage;
	}

	return UpperValue - LowerValue;
}

void SStaticMeshLODVisualizerWidget::SetNaniteEnabled(bool bNaniteEnabledIn)
{
	bNaniteEnabled = bNaniteEnabledIn;
}

void SStaticMeshLODVisualizerWidget::SetNaniteActive(bool bNaniteActiveIn)
{
	bNaniteActive = bNaniteActiveIn;
}

bool SStaticMeshLODVisualizerWidget::IsNaniteOverlayActive() const
{
	return bNaniteEnabled && bNaniteActive;
}

void SStaticMeshLODVisualizerWidget::SetMinimumLOD(int LODIndexIn)
{
	MinimumLOD = FMath::Clamp(LODIndexIn, 0, 7);;
}

void SStaticMeshLODVisualizerWidget::SetCurrentPlatform(FString PlatformName)
{
	CurrentPlatform = PlatformName;
}

#undef LOCTEXT_NAMESPACE