// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FLODVisualizationData
{
	float ScreenPercentage = 0.0;
	TAttribute<float> ActivationPercentage;
	TAttribute<EVisibility> Visibility;
	TAttribute<EVisibility> MinimalLODVisibility;
	TAttribute<FSlateColor> Color;
	TAttribute<const FSlateBrush*> ActiveBorder;
	TAttribute<FText> ToolTip;
};

class SVerticalBox;

class SStaticMeshLODVisualizerWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SStaticMeshLODVisualizerWidget) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments & InArgs);

	void SetScreenPercentage(float NewScreenPercentageIn);

	void SetLODPercentage(int32 LODIndex, float NewLODPercentageIn);

	void SetNumLODs(int NumLods);

	void SetActiveLOD(int LODIndexIn);

	void SetNaniteEnabled(bool bNaniteEnabledIn);

	void SetNaniteActive(bool bNaniteActiveIn);

	void SetMinimumLOD(int LODIndexIn);

	void SetCurrentPlatform(FString PlatformName);

protected:

	float ScreenPercentage = 1.f;
	int32 NumLODs = 1;
	int32 ActiveLOD = 0;
	int32 MinimumLOD = 0;
	FString CurrentPlatform = "NONE";

	bool bNaniteEnabled = false;
	bool bNaniteActive = false;

	TSharedPtr<SVerticalBox> LODSlotDisplay;

	TAttribute<float> SliderValue;
	TAttribute<float> PercentAbove;
	TAttribute<float> PercentBelow;

	TAttribute<EVisibility> NaniteOverlayVisibility;

	TArray<FLODVisualizationData> PerLODData;

	float ComputeLODSectionFill(int32 LODIndex);

	bool IsNaniteOverlayActive() const;
};