// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"


#include "Tests/AutomationCommon.h"
#include "StaticMeshEditorViewportClient.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if WITH_AUTOMATION_TESTS

namespace EditorViewButtonHelper
{
	/**
	 * The types of buttons that will be toggled on and off, if new buttons are made that want to be added, 
	 * all you need to do is add them to this list, and fill out the latent automation task below with how to toggle the button
	 */
	namespace EStaticMeshFlag
	{
		enum Type
		{
			Wireframe = 0,
			Vert,
			Grid,
			Bounds,
			Collision,
			Pivot,
			Normals,
			Tangents,
			Binormals,
			UV,
			Max //Do not go beyond this
		};
	}

	/**
	 * The struct we will be passing into the below Latent Automation Task
	 */
	struct PerformStaticMeshFlagParameters
	{
		FStaticMeshEditorViewportClient* ViewportClient;
		EStaticMeshFlag::Type CommandType;
	};

	static const TCHAR* GetStaticMeshFlagName(EStaticMeshFlag::Type InType)
	{
		switch(InType)
		{
		case EStaticMeshFlag::Wireframe:
			return TEXT("Wireframe");
		case EStaticMeshFlag::Vert:
			return TEXT("Vertex");
		case EStaticMeshFlag::Grid:
			return TEXT("Grid");
		case EStaticMeshFlag::Bounds:
			return TEXT("Bounds");
		case EStaticMeshFlag::Collision:
			return TEXT("Collision");
		case EStaticMeshFlag::Pivot:
			return TEXT("Pivot");
		case EStaticMeshFlag::Normals:
			return TEXT("Normals");
		case EStaticMeshFlag::Tangents:
			return TEXT("Tangents");
		case EStaticMeshFlag::Binormals:
			return TEXT("Binormals");
		case EStaticMeshFlag::UV:
			return TEXT("UV");
		default:
			break;
		}

		return TEXT("Unknown");
	}

	/**
	 * Used for toggling a certain button on/off based on the passed in viewport client and button type.
	 */
	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FPerformStaticMeshFlagToggle, PerformStaticMeshFlagParameters, AutomationParameters);

	bool EditorViewButtonHelper::FPerformStaticMeshFlagToggle::Update()
	{ 
		switch(AutomationParameters.CommandType)
		{
			case EStaticMeshFlag::Wireframe:
				if (AutomationParameters.ViewportClient->GetViewMode() != VMI_Wireframe)
				{
					AutomationParameters.ViewportClient->SetViewMode(VMI_Wireframe);
				}
				else
				{
					AutomationParameters.ViewportClient->SetViewMode(VMI_Lit);
				}
				break;
			case EStaticMeshFlag::Vert:
				if (!AutomationParameters.ViewportClient->EngineShowFlags.VertexColors)
				{
					AutomationParameters.ViewportClient->EngineShowFlags.SetVertexColors(true);
					AutomationParameters.ViewportClient->EngineShowFlags.SetPhysicalMaterialMasks(false);
					AutomationParameters.ViewportClient->EngineShowFlags.SetLighting(false);
					AutomationParameters.ViewportClient->EngineShowFlags.SetIndirectLightingCache(false);
				}
				else
				{
					AutomationParameters.ViewportClient->EngineShowFlags.SetVertexColors(false);
					AutomationParameters.ViewportClient->EngineShowFlags.SetLighting(true);
					AutomationParameters.ViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
				}
				break;
			case EStaticMeshFlag::Grid:
				AutomationParameters.ViewportClient->SetShowGrid();
				break;
			case EStaticMeshFlag::Bounds:
				AutomationParameters.ViewportClient->EngineShowFlags.SetBounds(!AutomationParameters.ViewportClient->EngineShowFlags.Bounds);
				break;
			case EStaticMeshFlag::Collision:
				AutomationParameters.ViewportClient->SetShowCollision();
				break;
			case EStaticMeshFlag::Pivot:
				AutomationParameters.ViewportClient->ToggleShowPivot();
				break;
			case EStaticMeshFlag::Normals:
				AutomationParameters.ViewportClient->ToggleShowNormals();
				break;
			case EStaticMeshFlag::Tangents:
				AutomationParameters.ViewportClient->ToggleShowTangents();
				break;
			case EStaticMeshFlag::Binormals:
				AutomationParameters.ViewportClient->ToggleShowBinormals();
				break;
			case EStaticMeshFlag::UV:
				AutomationParameters.ViewportClient->ToggleDrawUVOverlay();
				break;
			default:
				//Break out immediately.
				break;
		}
		return true;
	}

	/**
	* Close all asset editors
	*/
	DEFINE_LATENT_AUTOMATION_COMMAND(FCloseAllAssetEditorsCommand);

	bool FCloseAllAssetEditorsCommand::Update()
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

		return true;
	}
}

/**
 * Static mesh editor test
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStaticMeshEditorTest, "System.Editor.Content.Static Mesh Editor Test", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::NonNullRHI)

/**
 * Take screenshots of the SME window with each of the toolbar buttons toggled separately
 */
bool FStaticMeshEditorTest::RunTest(const FString& Parameters)
{
	TArray<FString> ButtonNamesToTestFor;
	FString LoadedObjectType = TEXT("EditorCylinder");
	FString MeshName;
	EditorViewButtonHelper::PerformStaticMeshFlagParameters AutomationParameters;
	WindowScreenshotParameters WindowParameters;

	//Pull from .ini the names of the buttons we want to test for
	GConfig->GetString( TEXT("AutomationTesting"), TEXT("EditorViewButtonsObject"), LoadedObjectType, GEngineIni);

	//Open static mesh in the editor and save out the window for the screenshots
	UObject* EditorMesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),
		NULL,
		*FString::Printf(TEXT("/Engine/EditorMeshes/%s.%s"), *LoadedObjectType, *LoadedObjectType),
		NULL,
		LOAD_None,
		NULL);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorMesh);

	{
		TArray<TSharedRef<SWindow>> AllWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(AllWindows);

		const FText ExpectedTitle = FText::FromString(LoadedObjectType);
		TSharedRef<SWindow>* Window = AllWindows.FindByPredicate([&](const TSharedRef<SWindow>& In) { return In->GetTitle().EqualTo(ExpectedTitle); });
		if (Window)
		{
			WindowParameters.CurrentWindow = *Window;
		}
	}

	if (!WindowParameters.CurrentWindow.IsValid())
	{
		AddError(TEXT("Could not find static mesh editor window"));
		return false;
	}

	//Grab the last opened Viewport (aka the asset manager)
	AutomationParameters.ViewportClient = static_cast<FStaticMeshEditorViewportClient*>(GEditor->GetAllViewportClients().Last());
	AutomationParameters.CommandType = EditorViewButtonHelper::EStaticMeshFlag::Wireframe; //Start it off at Wireframe because that is the first button we are attempting to press

	if (AutomationParameters.ViewportClient)
	{
		const FString BaseFileName = TEXT("StaticMeshEditorTest");
		FString StaticMeshTestName = FString::Printf(TEXT("%s/Base"), *BaseFileName);

		for(auto CurrentCommandType = 0; CurrentCommandType < EditorViewButtonHelper::EStaticMeshFlag::Max; ++CurrentCommandType)
		{
			StaticMeshTestName = FString::Printf(TEXT("%s/%s"), *BaseFileName, EditorViewButtonHelper::GetStaticMeshFlagName(AutomationParameters.CommandType));
			
			//Toggle the command for the button, take a screenshot, and then re-toggle the command for the button
			ADD_LATENT_AUTOMATION_COMMAND(EditorViewButtonHelper::FPerformStaticMeshFlagToggle(AutomationParameters));

			//Toggle the flag back off
			ADD_LATENT_AUTOMATION_COMMAND(EditorViewButtonHelper::FPerformStaticMeshFlagToggle(AutomationParameters));
			
			//Increment to the next enum, so we can loop through all the types
			AutomationParameters.CommandType = (EditorViewButtonHelper::EStaticMeshFlag::Type)((int)AutomationParameters.CommandType + 1);
		}

		ADD_LATENT_AUTOMATION_COMMAND(EditorViewButtonHelper::FCloseAllAssetEditorsCommand());
	}

	return true;
}

#endif // WITH_AUTOMATION_TESTS