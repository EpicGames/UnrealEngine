// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditor.h"
#include "StaticMeshEditorEdMode.h"
#include "AdvancedPreviewScene.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorGizmos/TransformGizmo.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "PhysicsEngine/BodySetupObjectTextFactory.h"
#include "Styling/AppStyle.h"
#include "EditorReimportHandler.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "StaticMeshEditorModule.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"

#include "SStaticMeshEditorViewport.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "IDetailCustomization.h"
#include "StaticMeshEditorTools.h"
#include "StaticMeshEditorActions.h"

#include "StaticMeshResources.h"
#include "BusyCursor.h"
#include "GeomFitUtils.h"
#include "EditorViewportCommands.h"
#include "ConvexDecompTool.h"

#include "MeshMergeModule.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Input/STextComboBox.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BodySetup.h"

#include "AdvancedPreviewSceneModule.h"

#include "ConvexDecompositionNotification.h"
#include "FbxMeshUtils.h"
#include "RawMesh.h"
#include "EditorViewportTabContent.h"
#include "EditorViewportLayout.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "EditorModeManager.h"
#include "StaticMeshEditorModeUILayer.h"
#include "AssetEditorModeManager.h"
#include "Engine/Selection.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "StaticMeshEditorViewportClient.h"
#include "Engine/StaticMeshSocket.h"
#include "AdvancedPreviewScene.h"
#include "SnappingUtils.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditor"

DEFINE_LOG_CATEGORY_STATIC(LogStaticMeshEditor, Log, All);

class FStaticMeshStatusMessageContext : public FScopedSlowTask
{
public:
	explicit FStaticMeshStatusMessageContext(const FText& InMessage)
		: FScopedSlowTask(0, InMessage)
	{
		UE_LOGF(LogStaticMesh, Log, "%ls", *InMessage.ToString());
		MakeDialog();
	}
};

namespace StaticMeshEditor
{
	static void PopulateCollisionMenu(UToolMenu* Menu)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("CollisionEditCollision", LOCTEXT("CollisionEditCollisionSection", "Edit Collision"));
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateSphereCollision);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateSphylCollision);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateBoxCollision);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateDOP10X);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateDOP10Y);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateDOP10Z);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateDOP18);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateDOP26);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ConvertBoxesToConvex);
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().RemoveCollision);

			Section.AddMenuEntry("DeleteCollision", FGenericCommands::Get().Delete, LOCTEXT("DeleteCollision", "Delete Selected Collision"), LOCTEXT("DeleteCollisionToolTip", "Deletes the selected Collision from the mesh."));
			Section.AddMenuEntry("DuplicateCollision", FGenericCommands::Get().Duplicate, LOCTEXT("DuplicateCollision", "Duplicate Selected Collision"), LOCTEXT("DuplicateCollisionToolTip", "Duplicates the selected Collision."));
			Section.AddMenuEntry("CopyCollision", FGenericCommands::Get().Copy, LOCTEXT("CopyCollision", "Copy Selected Collision"), LOCTEXT("CopyCollisionToolTip", "Copy the selected Collision to the clipboard."));
			Section.AddMenuEntry("PasteCollision", FGenericCommands::Get().Paste, LOCTEXT("PasteCollision", "Paste Copied Collision"), LOCTEXT("PasteCollisionToolTip", "Paste coppied Collision from the clipboard."));
		}

		{
			FToolMenuSection& Section = Menu->AddSection("CollisionAutoConvexCollision");
			Section.AddSeparator("MiscActionsSeparator");
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CreateAutoConvexCollision);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("CollisionCopy");
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().CopyCollisionFromSelectedMesh);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("MeshFindSource");
			Section.AddMenuEntry(FStaticMeshEditorCommands::Get().FindSource);
		}

		{
			FToolMenuSection& Section = Menu->AddSection("MeshChange");
			//Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ChangeMesh);
			Section.AddDynamicEntry("SaveGeneratedLODs", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				static auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.StaticMesh.EnableSaveGeneratedLODsInPackage"));
				if (CVar && CVar->GetValueOnGameThread() != 0)
				{
					InSection.AddMenuEntry(FStaticMeshEditorCommands::Get().SaveGeneratedLODs);
				}
			}));
		}
	}

	static TSharedPtr<FStaticMeshEditor> GetStaticMeshEditorFromMenuContext(UAssetEditorToolkitMenuContext* InContext)
	{
		if (InContext)
		{
			if (TSharedPtr<FAssetEditorToolkit> Toolkit = InContext->Toolkit.Pin())
			{
				// Note: This will not detect subclasses of StaticMeshEditor
				if (Toolkit->GetToolkitFName() == TEXT("StaticMeshEditor"))
				{
					return StaticCastSharedPtr<FStaticMeshEditor>(Toolkit);
				}
			}
		}

		return nullptr;
	}

	static TSharedPtr<FStaticMeshEditor> GetStaticMeshEditorFromMenuContext(UToolMenu* InMenu)
	{
		return GetStaticMeshEditorFromMenuContext(InMenu->FindContext<UAssetEditorToolkitMenuContext>());
	}

	static TSharedPtr<FStaticMeshEditor> GetStaticMeshEditorFromMenuContext(FToolMenuSection& InSection)
	{
		return GetStaticMeshEditorFromMenuContext(InSection.FindContext<UAssetEditorToolkitMenuContext>());
	}
}

const FName FStaticMeshEditor::ViewportTabId( TEXT( "StaticMeshEditor_Viewport" ) );
const FName FStaticMeshEditor::PropertiesTabId( TEXT( "StaticMeshEditor_Properties" ) );
const FName FStaticMeshEditor::SocketManagerTabId( TEXT( "StaticMeshEditor_SocketManager" ) );
const FName FStaticMeshEditor::CollisionTabId( TEXT( "StaticMeshEditor_Collision" ) );
const FName FStaticMeshEditor::PreviewSceneSettingsTabId( TEXT ("StaticMeshEditor_PreviewScene" ) );
const FName FStaticMeshEditor::SecondaryToolbarTabId( TEXT( "StaticMeshEditor_SecondaryToolbar" ) );

void FStaticMeshEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_StaticMeshEditor", "Static Mesh Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( ViewportTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_Viewport) )
		.SetDisplayName( LOCTEXT("ViewportTab", "Viewport") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner( PropertiesTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_Properties) )
		.SetDisplayName( LOCTEXT("PropertiesTab", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner( SocketManagerTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_SocketManager) )
		.SetDisplayName( LOCTEXT("SocketManagerTab", "Socket Manager") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.Tabs.SocketManager"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner( CollisionTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_Collision) )
		.SetDisplayName( LOCTEXT("CollisionTab", "Convex Decomposition") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.Tabs.ConvexDecomposition"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Hidden);

	InTabManager->RegisterTabSpawner( PreviewSceneSettingsTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_PreviewSceneSettings) )
		.SetDisplayName( LOCTEXT("PreviewSceneTab", "Preview Scene Settings") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	FTabSpawnerEntry& MenuEntry = InTabManager->RegisterTabSpawner( SecondaryToolbarTabId, FOnSpawnTab::CreateSP(this, &FStaticMeshEditor::SpawnTab_SecondaryToolbar) )
		.SetDisplayName( LOCTEXT("ToolbarTab", "Secondary Toolbar") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Toolbar.Icon"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Hidden);

	// Hide the menu item by default. It will be enabled only if the secondary toolbar is populated with extensions
	SecondaryToolbarEntry = &MenuEntry;
	SecondaryToolbarEntry->SetMenuType( ETabSpawnerMenuType::Hidden );

	OnRegisterTabSpawners().Broadcast(InTabManager);
}

void FStaticMeshEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( ViewportTabId );
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
	InTabManager->UnregisterTabSpawner( SocketManagerTabId );
	InTabManager->UnregisterTabSpawner( CollisionTabId );
	InTabManager->UnregisterTabSpawner( PreviewSceneSettingsTabId );
	InTabManager->UnregisterTabSpawner( SecondaryToolbarTabId );

	OnUnregisterTabSpawners().Broadcast(InTabManager);
}


FStaticMeshEditor::~FStaticMeshEditor()
{
	if (StaticMesh)
	{
		StaticMesh->GetOnMeshChanged().RemoveAll(this);
	}

 	if (ViewportTabContent.IsValid())
 	{
 		ViewportTabContent->OnViewportTabContentLayoutChanged().RemoveAll(this);
 	}

	OnStaticMeshEditorClosed().Broadcast();

#if USE_ASYNC_DECOMP
	/** If there is an active instance of the asynchronous convex decomposition interface, release it here. */
	if (GConvexDecompositionNotificationState)
	{
		GConvexDecompositionNotificationState->IsActive = false;
	}
	if (DecomposeMeshToHullsAsync)
	{
		DecomposeMeshToHullsAsync->Release();
	}
#endif
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	GEditor->UnregisterForUndo( this );
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
}

void FStaticMeshEditor::InitEditorForStaticMesh(UStaticMesh* ObjectToEdit)
{
	FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FStaticMeshEditor::OnPostReimport);

	// Support undo/redo
	ObjectToEdit->SetFlags( RF_Transactional );
	if (ObjectToEdit->GetNavCollision())
	{
		ObjectToEdit->GetNavCollision()->SetFlags(RF_Transactional);
	}

	GEditor->RegisterForUndo( this );

	// Register our commands. This will only register them if not previously registered
	FStaticMeshEditorCommands::Register();

	// Register to be notified when an object is reimported.
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddSP(this, &FStaticMeshEditor::OnObjectReimported);

	BindCommands();

	// The tab must be created before the viewport layout because the layout needs them
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;

	StaticMeshDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	FOnGetDetailCustomizationInstance LayoutCustomStaticMeshProperties = FOnGetDetailCustomizationInstance::CreateSP( this, &FStaticMeshEditor::MakeStaticMeshDetails );
	StaticMeshDetailsView->RegisterInstancedCustomPropertyLayout( UStaticMesh::StaticClass(), LayoutCustomStaticMeshProperties );

	StaticMesh = ObjectToEdit;

	IStaticMeshEditorModule* StaticMeshEditorModule = &FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
	StaticMeshEditorModule->OnStaticMeshEditorOpened().Broadcast(SharedThis(this));

}

void FStaticMeshEditor::InitStaticMeshEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStaticMesh* ObjectToEdit )
{
	InitEditorForStaticMesh(ObjectToEdit);

	TSharedRef<FTabManager::FStack> ExtentionTabStack(
		FTabManager::NewStack()
		->SetSizeCoefficient(0.3f)
		->AddTab(SocketManagerTabId, ETabState::OpenedTab)
		->AddTab(CollisionTabId, ETabState::ClosedTab));
	//Let additional extensions dock themselves to this TabStack of tools
	OnStaticMeshEditorDockingExtentionTabs().Broadcast(ExtentionTabStack);

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_StaticMeshEditor_Layout_v6" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab(ViewportTabId, ETabState::OpenedTab)
				->SetHideTabWell( true )
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.25f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.7f)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
					->AddTab(SocketManagerTabId, ETabState::OpenedTab)
					->SetForegroundTab(PropertiesTabId) 
				)
				->Split
				(
					ExtentionTabStack
				)
			)
		)
	);


	// Add any extenders specified by the UStaticMeshEditorUISubsystem
	IStaticMeshEditorModule* StaticMeshEditorModule = &FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
	FLayoutExtender LayoutExtender;
	StaticMeshEditorModule->OnRegisterLayoutExtensions().Broadcast(LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(LayoutExtender);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, StaticMeshEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultToolbar, bCreateDefaultStandaloneMenu, ObjectToEdit );

	GetEditorModeManager().SetDefaultMode(FStaticMeshEditorEdMode::Id);
	GetEditorModeManager().ActivateDefaultMode();

	GetEditorModeManager().SetSupportsViewportITF(true);

	if (UModeManagerInteractiveToolsContext* ToolsContext = GetEditorModeManager().GetInteractiveToolsContext())
	{
		ToolsContext->SetViewportInteractionsEnabled(true);
	}

	StaticMeshDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([this]
	{
		return GetOpenMethod() == EAssetOpenMethod::Edit;
	}));
	
	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShareable(new FStaticMeshEditorModeUILayer(PinnedToolkitHost.Get()));

	ExtendMenu();
	ExtendToolBar();
	RegenerateMenusAndToolbars();
	GenerateSecondaryToolbar();
}


void FStaticMeshEditor::PostInitAssetEditor()
{
	// (Copied from FUVEditorToolkit::PostInitAssetEditor)
	// We need the viewport client to start out focused, or else it won't get ticked until we click inside it.
	if (TSharedPtr<SStaticMeshEditorViewport> StaticMeshViewport = GetStaticMeshViewport())
	{
		FStaticMeshEditorViewportClient& ViewportClient = StaticMeshViewport->GetViewportClient();
		ViewportClient.ReceivedFocus(ViewportClient.Viewport);
	}

	// Static mesh editor code generally assumes the SocketManager exists, so make sure it does (in case the tab manager / sockets window hasn't already done so)
	InitSocketManager();
}

void FStaticMeshEditor::OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source)
{
	FAssetEditorToolkit::OnBuildViewportInteractions(Source);
	FAdvancedPreviewScene::AddPreviewSceneInteractions(Source);
}

void FStaticMeshEditor::GenerateSecondaryToolbar()
{
	// Generate the secondary toolbar only if there are registered extensions
	TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(SecondaryToolbarTabId);

	TSharedPtr<FExtender> Extender = FExtender::Combine(SecondaryToolbarExtenders);
	if (Extender->NumExtensions() == 0)
	{
		// If the tab was previously opened, close it since it's now empty
		if (Tab)
		{
			Tab->RemoveTabFromParent();
		}
		return;
	}

	const bool bIsFocusable = true;

	FToolBarBuilder ToolbarBuilder(GetToolkitCommands(), FMultiBoxCustomization::AllowCustomization(GetToolkitFName()), Extender);
	ToolbarBuilder.SetIsFocusable(bIsFocusable);
	ToolbarBuilder.BeginSection("Extensions");
	{
		// The secondary toolbar itself is empty but will be populated by the extensions when EndSection is called.
		// The section name helps in the extenders positioning.
	}
	ToolbarBuilder.EndSection();

	// Setup the secondary toolbar menu entry
	SecondaryToolbarEntry->SetMenuType(ETabSpawnerMenuType::Enabled);
	SecondaryToolbarEntry->SetDisplayName(SecondaryToolbarDisplayName);

	SecondaryToolbar =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				ToolbarBuilder.MakeWidget()
			]
		];

	if (SecondaryToolbarWidgetContent.IsValid())
	{
		SecondaryToolbarWidgetContent->SetContent(SecondaryToolbar.ToSharedRef());
	}

	if (!Tab)
	{
		// By default, the tab is closed but we want it to be opened when it is populated
		Tab = TSharedPtr<SDockTab>(TabManager->TryInvokeTab(SecondaryToolbarTabId));
	}

	// Override the display name if it was set
	if (!SecondaryToolbarDisplayName.IsEmpty())
	{
		Tab->SetLabel(SecondaryToolbarDisplayName);
	}
}

void FStaticMeshEditor::AddSecondaryToolbarExtender(TSharedPtr<FExtender> Extender)
{
	SecondaryToolbarExtenders.AddUnique(Extender);
}

void FStaticMeshEditor::RemoveSecondaryToolbarExtender(TSharedPtr<FExtender> Extender)
{
	SecondaryToolbarExtenders.Remove(Extender);
}

void FStaticMeshEditor::SetSecondaryToolbarDisplayName(FText DisplayName)
{
	SecondaryToolbarDisplayName = DisplayName;
}

TSharedRef<IDetailCustomization> FStaticMeshEditor::MakeStaticMeshDetails()
{
	TSharedRef<FStaticMeshDetails> NewDetails = MakeShareable( new FStaticMeshDetails( *this ) );
	StaticMeshDetails = NewDetails;
	return NewDetails;
}

void FStaticMeshEditor::ExtendMenu()
{
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.StaticMeshEditor.MainMenu.Edit");
		FToolMenuSection& Section = Menu->AddSection("Sockets", LOCTEXT("EditStaticMeshSockets", "Sockets"));
		Section.InsertPosition = FToolMenuInsert("EditHistory", EToolMenuInsertType::After);
		Section.AddMenuEntry("DeleteSocket", FGenericCommands::Get().Delete, LOCTEXT("DeleteSocket", "Delete Socket"), LOCTEXT("DeleteSocketToolTip", "Deletes the selected socket from the mesh."));
		Section.AddMenuEntry("DuplicateSocket", FGenericCommands::Get().Duplicate, LOCTEXT("DuplicateSocket", "Duplicate Socket"), LOCTEXT("DuplicateSocketToolTip", "Duplicates the selected socket."));
	}

	{
		if (!UToolMenus::Get()->IsMenuRegistered("StaticMeshEditor.Collision"))
		{
			StaticMeshEditor::PopulateCollisionMenu(UToolMenus::Get()->RegisterMenu("StaticMeshEditor.Collision"));
		}

		if (!UToolMenus::Get()->IsMenuRegistered("AssetEditor.StaticMeshEditor.MainMenu.Collision"))
		{
			UToolMenus::Get()->RegisterMenu("AssetEditor.StaticMeshEditor.MainMenu.Collision", "StaticMeshEditor.Collision");
		}

		{
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.StaticMeshEditor.MainMenu");

			FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);

			Section.AddDynamicEntry("CollisionDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (TSharedPtr<FStaticMeshEditor> StaticMeshEditor = StaticMeshEditor::GetStaticMeshEditorFromMenuContext(InSection))
				{
					FToolMenuEntry& Entry = InSection.AddSubMenu("Collision",
						LOCTEXT("StaticMeshEditorCollisionMenu", "Collision"),
						LOCTEXT("StaticMeshEditorCollisionMenu_ToolTip", "Opens a menu with commands for editing this mesh's collision"),
						FNewToolMenuChoice());
						
					Entry.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);
				}
			}));
			
			
		}
	}

	IStaticMeshEditorModule* StaticMeshEditorModule = &FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>( "StaticMeshEditor" );
	AddMenuExtender(StaticMeshEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	UToolMenu* AssetMenu = UToolMenus::Get()->ExtendMenu("AssetEditor.StaticMeshEditor.MainMenu.Asset");
	FToolMenuSection& AssetSection = AssetMenu->FindOrAddSection("AssetEditorActions");
	FToolMenuEntry& Entry = AssetSection.AddDynamicEntry("AssetManagerEditorStaticMeshCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		InSection.AddMenuEntry(FStaticMeshEditorCommands::Get().FindSource);
		InSection.AddMenuEntry(FStaticMeshEditorCommands::Get().SetDrawAdditionalData);
		InSection.AddMenuEntry(FStaticMeshEditorCommands::Get().BakeMaterials);
		InSection.AddDynamicEntry("SaveGeneratedLODs", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			static auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.StaticMesh.EnableSaveGeneratedLODsInPackage"));
			if (CVar && CVar->GetValueOnGameThread() != 0)
			{
				InSection.AddMenuEntry(FStaticMeshEditorCommands::Get().SaveGeneratedLODs);
			}
		}));
	}));
}

void FStaticMeshEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( StaticMesh );
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_Viewport( const FSpawnTabArgs& Args )
{
	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	TWeakPtr<IStaticMeshEditor> WeakSharedThis(SharedThis(this));
	MakeViewportFunc = [WeakSharedThis](const FAssetEditorViewportConstructionArgs& InArgs)
	{
		return SNew(SStaticMeshEditorViewport)
			.StaticMeshEditor(WeakSharedThis);
	};

	// Create a new tab
	ViewportTabContent = MakeShareable(new FEditorViewportTabContent());
	ViewportTabContent->OnViewportTabContentLayoutChanged().AddRaw(this, &FStaticMeshEditor::OnEditorLayoutChanged);

	const FString LayoutId = FString("StaticMeshEditorViewport");
	ViewportTabContent->Initialize(MakeViewportFunc, DockableTab, LayoutId);

	GetStaticMeshViewport()->SetParentTab(DockableTab);

	return DockableTab;
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_Properties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == PropertiesTabId );

	return SNew(SDockTab)
		.Label( LOCTEXT("StaticMeshProperties_TabTitle", "Details") )
		[
			StaticMeshDetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_SocketManager( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == SocketManagerTabId );

	InitSocketManager();

	return SNew(SDockTab)
		.Label( LOCTEXT("StaticMeshSocketManager_TabTitle", "Socket Manager") )
		[
			SocketManager.ToSharedRef()
		];
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_Collision( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == CollisionTabId );

	if (!ConvexDecomposition)
	{
		SAssignNew( ConvexDecomposition, SConvexDecomposition )
			.StaticMeshEditorPtr(SharedThis(this));
	}

	return SNew(SDockTab)
		.Label( LOCTEXT("StaticMeshConvexDecomp_TabTitle", "Convex Decomposition") )
		[
			ConvexDecomposition.ToSharedRef()
		];
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_PreviewSceneSettings( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == PreviewSceneSettingsTabId );
	return SAssignNew(PreviewSceneDockTab, SDockTab)
		.Label( LOCTEXT("StaticMeshPreviewScene_TabTitle", "Preview Scene Settings") )
		[
			AdvancedPreviewSettingsWidget.IsValid() ? AdvancedPreviewSettingsWidget.ToSharedRef() : SNullWidget::NullWidget
		];
}

TSharedRef<SDockTab> FStaticMeshEditor::SpawnTab_SecondaryToolbar( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == SecondaryToolbarTabId );

	FText TabLabel = !SecondaryToolbarDisplayName.IsEmpty() ? SecondaryToolbarDisplayName : LOCTEXT("SecondaryToolbar_TabTitle", "Secondary Toolbar");

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label( TabLabel )
		.ShouldAutosize( true )
		[
			SAssignNew(SecondaryToolbarWidgetContent, SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		];

	if ( SecondaryToolbar.IsValid() )
	{
		SecondaryToolbarWidgetContent->SetContent( SecondaryToolbar.ToSharedRef() );
	}
	
	return SpawnedTab;
}

TSharedPtr<SStaticMeshEditorViewport> FStaticMeshEditor::GetStaticMeshViewport() const
{
	if (ViewportTabContent.IsValid())
	{
		// we can use static cast here b/c we know in this editor we will have a static mesh viewport 
		return StaticCastSharedPtr<SStaticMeshEditorViewport>(ViewportTabContent->GetFirstViewport());
	}

	return TSharedPtr<SStaticMeshEditorViewport>();
}

void FStaticMeshEditor::OnEditorLayoutChanged()
{
	SetEditorMesh(StaticMesh);

	BuildSubTools();

	bool LocalDrawGrids = false;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> CheckShowGridFunc =
		[this, &LocalDrawGrids](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		LocalDrawGrids |= StaticMeshEditorViewportClient.IsSetShowGridChecked();
	};

	ViewportTabContent->PerformActionOnViewports(CheckShowGridFunc);

	bDrawGrids = LocalDrawGrids;

	OnPreviewSceneChangedDelegate.Broadcast(GetStaticMeshViewport()->GetPreviewScene());
}

void FStaticMeshEditor::BindCommands()
{
	const FStaticMeshEditorCommands& Commands = FStaticMeshEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();


	UICommandList->MapAction( FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP( this, &FStaticMeshEditor::DeleteSelected ),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanDeleteSelected));

	UICommandList->MapAction( FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP( this, &FStaticMeshEditor::UndoAction ) );

	UICommandList->MapAction( FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP( this, &FStaticMeshEditor::RedoAction ) );

	UICommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::DuplicateSelected),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanDuplicateSelected),
		FIsActionChecked());

	UICommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::CopySelected),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanCopySelected));

	UICommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::PasteCopied),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanPasteCopied));

	UICommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::RequestRenameSelectedSocket),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanRenameSelected),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.CreateDOP10X,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::GenerateKDop, KDopDir10X, (uint32)10));

	UICommandList->MapAction(
		Commands.CreateDOP10Y,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::GenerateKDop, KDopDir10Y, (uint32)10));

	UICommandList->MapAction(
		Commands.CreateDOP10Z,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::GenerateKDop, KDopDir10Z, (uint32)10));

	UICommandList->MapAction(
		Commands.CreateDOP18,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::GenerateKDop, KDopDir18, (uint32)18));

	UICommandList->MapAction(
		Commands.CreateDOP26,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::GenerateKDop, KDopDir26, (uint32)26));

	UICommandList->MapAction(
		Commands.CreateBoxCollision,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnCollisionBox));

	UICommandList->MapAction(
		Commands.CreateSphereCollision,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnCollisionSphere));

	UICommandList->MapAction(
		Commands.CreateSphylCollision,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnCollisionSphyl));

	UICommandList->MapAction(Commands.ToggleShowNormals,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowNormals),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowNormalsChecked));

	UICommandList->MapAction(Commands.ToggleShowTangents,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowTangents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowTangentsChecked));

	UICommandList->MapAction(Commands.ToggleShowBinormals,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowBinormals),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowBinormalsChecked));

	UICommandList->MapAction(Commands.ToggleShowPivots,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowPivots),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowPivotsChecked));

	UICommandList->MapAction(Commands.ToggleShowVertices,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowVertices),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowVerticesChecked));

	UICommandList->MapAction(Commands.ToggleShowGrids,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowGrids),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowGridsChecked));

	UICommandList->MapAction(Commands.ToggleShowBounds,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowBounds),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowBoundsChecked));

	UICommandList->MapAction(Commands.ToggleShowSimpleCollisions,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowSimpleCollisions),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowSimpleCollisionsChecked));

	UICommandList->MapAction(Commands.ToggleShowComplexCollisions,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowComplexCollisions),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowComplexCollisionsChecked));

	UICommandList->MapAction(Commands.ToggleShowSockets,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowSockets),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowSocketsChecked));

	UICommandList->MapAction(Commands.ToggleShowWireframes,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowWireframes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowWireframesChecked));

	UICommandList->MapAction(Commands.ToggleShowVertexColors,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleShowVertexColors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsShowVertexColorsChecked));

	UICommandList->MapAction(
		Commands.RemoveCollision,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnRemoveCollision),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanRemoveCollision));

	UICommandList->MapAction(
		Commands.ConvertBoxesToConvex,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnConvertBoxToConvexCollision));

	UICommandList->MapAction(
		Commands.CopyCollisionFromSelectedMesh,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnCopyCollisionFromSelectedStaticMesh),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanCopyCollisionFromSelectedStaticMesh));

	// Mesh menu
	UICommandList->MapAction(
		Commands.FindSource,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ExecuteFindInExplorer),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanExecuteSourceCommands));

	UICommandList->MapAction(
		Commands.ChangeMesh,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnChangeMesh),
		FCanExecuteAction::CreateSP(this, &FStaticMeshEditor::CanChangeMesh));

	UICommandList->MapAction(
		Commands.SaveGeneratedLODs,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnSaveGeneratedLODs));

	{
		constexpr bool bWithNewFileTrue = true;
		constexpr bool bWithNewFileFalse = false;
		constexpr bool bWithDialogTrue = true;
		constexpr bool bWithDialogFalse = false;
		UICommandList->MapAction(
			Commands.ReimportMesh,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportMesh, FReimportParameters(bWithNewFileFalse, bWithDialogFalse)));

		UICommandList->MapAction(
			Commands.ReimportAllMesh,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportAllMesh, FReimportParameters(bWithNewFileFalse, bWithDialogFalse)));

		UICommandList->MapAction(
			Commands.ReimportMeshWithNewFile,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportMesh, FReimportParameters(bWithNewFileTrue, bWithDialogFalse)));

		UICommandList->MapAction(
			Commands.ReimportAllMeshWithNewFile,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportAllMesh, FReimportParameters(bWithNewFileTrue, bWithDialogFalse)));

		UICommandList->MapAction(
			Commands.ReimportMeshWithDialog,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportMesh, FReimportParameters(bWithNewFileFalse, bWithDialogTrue)));

		UICommandList->MapAction(
			Commands.ReimportAllMeshWithDialog,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportAllMesh, FReimportParameters(bWithNewFileFalse, bWithDialogTrue)));

		UICommandList->MapAction(
			Commands.ReimportMeshWithNewFileWithDialog,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportMesh, FReimportParameters(bWithNewFileTrue, bWithDialogTrue)));

		UICommandList->MapAction(
			Commands.ReimportAllMeshWithNewFileWithDialog,
			FExecuteAction::CreateSP(this, &FStaticMeshEditor::HandleReimportAllMesh, FReimportParameters(bWithNewFileTrue, bWithDialogTrue)));
	}

	// Collision Menu
	UICommandList->MapAction(
		Commands.CreateAutoConvexCollision,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::OnConvexDecomposition));

	// Viewport Camera
	UICommandList->MapAction(
		Commands.ResetCamera,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ResetCamera));

	// Draw additional data
	UICommandList->MapAction(
		Commands.SetDrawAdditionalData,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::ToggleDrawAdditionalData),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStaticMeshEditor::IsDrawAdditionalDataChecked));

	// Bake Materials
	UICommandList->MapAction(
		Commands.BakeMaterials,
		FExecuteAction::CreateSP(this, &FStaticMeshEditor::BakeMaterials),
		FCanExecuteAction::CreateLambda([this]()
		{
			return GetOpenMethod() != EAssetOpenMethod::View;
		}),
		FIsActionChecked());
}

void FStaticMeshEditor::ExtendToolBar()
{
	if (!UToolMenus::Get()->IsMenuRegistered("AssetEditor.StaticMeshEditor.ToolBar.Collision"))
	{
		UToolMenus::Get()->RegisterMenu("AssetEditor.StaticMeshEditor.ToolBar.Collision", "StaticMeshEditor.Collision");
	}

	// Toolbar
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.StaticMeshEditor.ToolBar");

		{
			FToolMenuSection& Section = Menu->AddSection("Mesh");
			Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

			Section.AddDynamicEntry("MeshDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (TSharedPtr<FStaticMeshEditor> StaticMeshEditor = StaticMeshEditor::GetStaticMeshEditorFromMenuContext(InSection))
				{
					//Reimport without dialog
					{
						auto ConstructReimportContextMenu = [](UToolMenu* InMenu)
							{
								FToolMenuSection& Section = InMenu->AddSection("Reimport");
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportMesh);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportMeshWithNewFile);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportAllMesh);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportAllMeshWithNewFile);
							};

						FToolMenuEntry& ReimportMeshEntry = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FStaticMeshEditorCommands::Get().ReimportMesh));
						ReimportMeshEntry.StyleNameOverride = "CalloutToolbar";

						FToolMenuEntry& ReimportContextMenuEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
							"ReimportContextMenu",
							FUIAction(),
							FNewToolMenuDelegate::CreateLambda(ConstructReimportContextMenu),
							TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(),
							true
						));
						ReimportContextMenuEntry.StyleNameOverride = "CalloutToolbar";
					}

					//Reimport force dialog
					{
						auto ConstructReimportWithDialogContextMenu = [](UToolMenu* InMenu)
							{
								FToolMenuSection& Section = InMenu->AddSection("Reimport");
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportMeshWithDialog);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportMeshWithNewFileWithDialog);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportAllMeshWithDialog);
								Section.AddMenuEntry(FStaticMeshEditorCommands::Get().ReimportAllMeshWithNewFileWithDialog);
							};
						FToolMenuEntry& ReimportMeshWithDialogEntry = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FStaticMeshEditorCommands::Get().ReimportMeshWithDialog));
						ReimportMeshWithDialogEntry.StyleNameOverride = "CalloutToolbar";

						FToolMenuEntry& ReimportWithDialogContextMenuEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
							"ReimportWithDialogContextMenu",
							FUIAction(),
							FNewToolMenuDelegate::CreateLambda(ConstructReimportWithDialogContextMenu),
							TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(),
							true
						));
						ReimportWithDialogContextMenuEntry.StyleNameOverride = "CalloutToolbar";
					}

				}
			}));

		}

		{
			FToolMenuSection& Section = Menu->AddSection("Command");
			Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

			Section.AddDynamicEntry("MeshDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (TSharedPtr<FStaticMeshEditor> StaticMeshEditor = StaticMeshEditor::GetStaticMeshEditorFromMenuContext(InSection))
				{
					FToolMenuEntry& CollisionEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
					"Collision",
					FUIAction(),
					FNewToolMenuChoice(), // let registered menu be looked up by name "AssetEditor.StaticMeshEditor.ToolBar.Collision"
					LOCTEXT("Collision_Label", "Collision"),
					LOCTEXT("Collision_Tooltip", "Collision drawing options"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.SetShowCollision")
					));
					
					CollisionEntry.StyleNameOverride = "CalloutToolbar";
				}
			}));

			

			Section.AddDynamicEntry("UVToolbarDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (TSharedPtr<FStaticMeshEditor> StaticMeshEditor = StaticMeshEditor::GetStaticMeshEditorFromMenuContext(InSection))
				{
					FToolMenuEntry& UVToolbarEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
						"UVToolbar",
						FUIAction(),
						FNewToolMenuDelegate::CreateSP(StaticMeshEditor.ToSharedRef(), &FStaticMeshEditor::GenerateUVChannelComboList),
						LOCTEXT("UVToolbarText", "UV"),
						LOCTEXT("UVToolbarTooltip", "Toggles display of the static mesh's UVs for the specified channel."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.SetDrawUVs")
					));
					UVToolbarEntry.StyleNameOverride = "CalloutToolbar";
				}
			}));
		}
	}


	// Extensions are currently disabled in read-only mode, if they are desired to be added in the future we should move this code to the individual extensions
	if(GetOpenMethod() == EAssetOpenMethod::Edit)
	{
		IStaticMeshEditorModule* StaticMeshEditorModule = &FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");

		TArray<IStaticMeshEditorModule::FStaticMeshEditorToolbarExtender> ToolbarExtenderDelegates = StaticMeshEditorModule->GetAllStaticMeshEditorToolbarExtenders();
		for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
		{
			if (ToolbarExtenderDelegate.IsBound())
			{
				AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
			}
		}

		EditorToolbarExtender = StaticMeshEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
		AddToolbarExtender(EditorToolbarExtender);
		AddSecondaryToolbarExtender(StaticMeshEditorModule->GetSecondaryToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	}

}

void FStaticMeshEditor::BuildSubTools()
{
	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");

	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
	Delegates.Add({ OnPreviewSceneChangedDelegate });
	AdvancedPreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(GetStaticMeshViewport()->GetPreviewScene(), nullptr, TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>(),  TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>(), Delegates);

	if (PreviewSceneDockTab.IsValid())
	{
		PreviewSceneDockTab.Pin()->SetContent(AdvancedPreviewSettingsWidget.ToSharedRef());
	}
}

FName FStaticMeshEditor::GetToolkitFName() const
{
	return FName("StaticMeshEditor");
}

FText FStaticMeshEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "StaticMesh Editor");
}

FString FStaticMeshEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "StaticMesh ").ToString();
}

FLinearColor FStaticMeshEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}

UStaticMeshComponent* FStaticMeshEditor::GetStaticMeshComponent() const
{
	return GetStaticMeshViewport().IsValid() ? GetStaticMeshViewport()->GetStaticMeshComponent() : nullptr;
}

void FStaticMeshEditor::SetSelectedSocket(UStaticMeshSocket* InSelectedSocket)
{
	SocketManager->SetSelectedSocket(InSelectedSocket);
}

void FStaticMeshEditor::AddSelectedSocket(UStaticMeshSocket* InSelectedSocket)
{
	SocketManager->AddSelectedSocket(InSelectedSocket);
}

void FStaticMeshEditor::RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket)
{
	SocketManager->RemoveSelectedSocket(InSelectedSocket);
}

UStaticMeshSocket* FStaticMeshEditor::GetSelectedSocket() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SocketManager.IsValid() ? SocketManager->GetSelectedSocket() : nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<UStaticMeshSocket*> FStaticMeshEditor::GetSelectedSockets() const
{
	return SocketManager.IsValid() ? SocketManager->GetSelectedSockets() : TArray<UStaticMeshSocket*>();
}

bool FStaticMeshEditor::HasSelectedSockets() const
{
	return SocketManager.IsValid() ? SocketManager->HasSelectedSockets() : false;
}

void FStaticMeshEditor::DuplicateSelectedSocket()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SocketManager->DuplicateSelectedSocket();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FStaticMeshEditor::DuplicateSelectedSockets()
{
	SocketManager->DuplicateSelectedSockets();
}

void FStaticMeshEditor::RequestRenameSelectedSocket()
{
	SocketManager->RequestRenameSelectedSocket();
}

void FStaticMeshEditor::InitSocketManager()
{
	if (!SocketManager)
	{
		FSimpleDelegate OnSocketSelectionChanged = FSimpleDelegate::CreateSP(SharedThis(this), &FStaticMeshEditor::OnSocketSelectionChanged);

		SocketManager = ISocketManager::CreateSocketManager(SharedThis(this), OnSocketSelectionChanged);
	}
}

bool FStaticMeshEditor::IsPrimValid(const FPrimData& InPrimData) const
{
	if (StaticMesh->GetBodySetup())
	{
		const FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

		switch (InPrimData.PrimType)
		{
		case EAggCollisionShape::Sphere:
			return AggGeom->SphereElems.IsValidIndex(InPrimData.PrimIndex);
		case EAggCollisionShape::Box:
			return AggGeom->BoxElems.IsValidIndex(InPrimData.PrimIndex);
		case EAggCollisionShape::Sphyl:
			return AggGeom->SphylElems.IsValidIndex(InPrimData.PrimIndex);
		case EAggCollisionShape::Convex:
			return AggGeom->ConvexElems.IsValidIndex(InPrimData.PrimIndex);
		case EAggCollisionShape::LevelSet:
			return AggGeom->LevelSetElems.IsValidIndex(InPrimData.PrimIndex);
		}
	}
	return false;
}

bool FStaticMeshEditor::HasSelectedPrims() const
{
	return (SelectedPrims.Num() > 0 ? true : false);
}

const TArray<IStaticMeshEditor::FPrimData>& FStaticMeshEditor::GetSelectedPrims() const
{
	return SelectedPrims;
}

void FStaticMeshEditor::AddSelectedPrim(const FPrimData& InPrimData, bool bClearSelection)
{
	check(IsPrimValid(InPrimData));

	// Enable collision, if not already
	if( !GetStaticMeshViewport()->GetViewportClient().IsShowSimpleCollisionChecked() )
	{
		GetStaticMeshViewport()->GetViewportClient().ToggleShowSimpleCollision();
	}

	if( bClearSelection )
	{
		ClearSelectedPrims();
	}
	SelectedPrims.Add(InPrimData);
}

void FStaticMeshEditor::RemoveSelectedPrim(const FPrimData& InPrimData)
{
	SelectedPrims.Remove(InPrimData);
}

void FStaticMeshEditor::RemoveInvalidPrims()
{
	for (int32 PrimIdx = SelectedPrims.Num() - 1; PrimIdx >= 0; PrimIdx--)
	{
		FPrimData& PrimData = SelectedPrims[PrimIdx];

		if (!IsPrimValid(PrimData))
		{
			SelectedPrims.RemoveAt(PrimIdx);
		}
	}
}

bool FStaticMeshEditor::IsSelectedPrim(const FPrimData& InPrimData) const
{
	return SelectedPrims.Contains(InPrimData);
}

void FStaticMeshEditor::ClearSelectedPrims()
{
	SelectedPrims.Empty();
}

void FStaticMeshEditor::DuplicateSelectedPrims(const FVector* InOffset)
{
	if (SelectedPrims.Num() > 0)
	{
		check(StaticMesh->GetBodySetup());

		FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

		GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_DuplicateSelectedPrims", "Duplicate Collision"));
		StaticMesh->GetBodySetup()->Modify();

		//Clear the cache (PIE may have created some data), create new GUID
		StaticMesh->GetBodySetup()->InvalidatePhysicsData();

		for (int32 PrimIdx = 0; PrimIdx < SelectedPrims.Num(); PrimIdx++)
		{
			FPrimData& PrimData = SelectedPrims[PrimIdx];

			check(IsPrimValid(PrimData));
			switch (PrimData.PrimType)
			{
			case EAggCollisionShape::Sphere:
				{
					const FKSphereElem SphereElem = AggGeom->SphereElems[PrimData.PrimIndex];
					PrimData.PrimIndex = AggGeom->SphereElems.Add(SphereElem);
				}
				break;
			case EAggCollisionShape::Box:
				{
					const FKBoxElem BoxElem = AggGeom->BoxElems[PrimData.PrimIndex];
					PrimData.PrimIndex = AggGeom->BoxElems.Add(BoxElem);
				}
				break;
			case EAggCollisionShape::Sphyl:
				{
					const FKSphylElem SphylElem = AggGeom->SphylElems[PrimData.PrimIndex];
					PrimData.PrimIndex = AggGeom->SphylElems.Add(SphylElem);
				}
				break;
			case EAggCollisionShape::Convex:
				{
					const FKConvexElem ConvexElem = AggGeom->ConvexElems[PrimData.PrimIndex];
					PrimData.PrimIndex = AggGeom->ConvexElems.Add(ConvexElem);
				}
				break;
			case EAggCollisionShape::LevelSet:
				{
					const FKLevelSetElem LevelSetElem = AggGeom->LevelSetElems[PrimData.PrimIndex];
					PrimData.PrimIndex = AggGeom->LevelSetElems.Add(LevelSetElem);
				}
				break;
			}

			// If specified, offset the duplicate by a specific amount
			if (InOffset)
			{
				FTransform PrimTransform = GetPrimTransform(PrimData);
				FVector PrimLocation = PrimTransform.GetLocation();
				PrimLocation += *InOffset;
				PrimTransform.SetLocation(PrimLocation);
				SetPrimTransform(PrimData, PrimTransform);
			}
		}

		// refresh collision change back to staticmesh components
		RefreshCollisionChange(*StaticMesh);

		GEditor->EndTransaction();

		// Mark staticmesh as dirty, to help make sure it gets saved.
		StaticMesh->MarkPackageDirty();

		// Update views/property windows
		GetStaticMeshViewport()->RefreshViewport();

		StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
	}
}

int32 FStaticMeshEditor::CopySelectedPrims() const
{
	int32 OutNumPrimsCopied = 0;
	if (CanCopySelected())
	{
		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		// Make a temp bodysetup to house all the selected shapes
		UBodySetup* NewBodySetup = NewObject<UBodySetup>();
		NewBodySetup->AddToRoot();
		if (StaticMesh)
		{
			if (const UBodySetup* OldBodySetup = StaticMesh->GetBodySetup())
			{
				const FKAggregateGeom& AggGeom = OldBodySetup->AggGeom;
				for (const FPrimData& Prim : SelectedPrims)
				{
					if (IsPrimValid(Prim))
					{
						if (NewBodySetup->AddCollisionElemFrom(AggGeom, Prim.PrimType, Prim.PrimIndex))
						{
							++OutNumPrimsCopied;
						}
					}
				}
			}
		}

		// Export the new bodysetup to the clipboard as text
		if (OutNumPrimsCopied > 0)
		{
			FStringOutputDevice Archive;
			const FExportObjectInnerContext Context;
			UExporter::ExportToOutputDevice(&Context, NewBodySetup, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
			FString ExportedText = Archive;
			FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
		}

		// Allow the temp bodysetup to get deleted by garbage collection
		NewBodySetup->RemoveFromRoot();
	}

	return OutNumPrimsCopied;
}

int32 FStaticMeshEditor::PasteCopiedPrims()
{
	int32 OutNumPrimsPasted = 0;
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	if (!TextToImport.IsEmpty())
	{
		UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/StaticMeshEditor/Transient"), RF_Transient);
		TempPackage->AddToRoot();
		{
			// Turn the text buffer into objects
			FBodySetupObjectTextFactory Factory;
			Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
			if (Factory.NewBodySetups.Num() > 0)
			{
				if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
				{
					GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_PasteCopiedPrims", "Paste Collision"));
					BodySetup->Modify();
					BodySetup->InvalidatePhysicsData();

					// Copy primitives from each bodysetup that was pasted
					for (const UBodySetup* NewBodySetup : Factory.NewBodySetups)
					{
						BodySetup->AddCollisionFrom(NewBodySetup->AggGeom);
						OutNumPrimsPasted += NewBodySetup->AggGeom.GetElementCount();
					}

					RefreshCollisionChange(*StaticMesh);
					GEditor->EndTransaction();
					StaticMesh->MarkPackageDirty();
					GetStaticMeshViewport()->RefreshViewport();
					StaticMesh->SetCustomizedCollision(true);
				}
			}
		}

		// Remove the temp package from the root now that it has served its purpose
		TempPackage->RemoveFromRoot();
	}

	return OutNumPrimsPasted;
}

void FStaticMeshEditor::TranslateSelectedPrims(const FVector& InDrag)
{
	check(StaticMesh->GetBodySetup());
	StaticMesh->GetBodySetup()->InvalidatePhysicsData();

	for (int32 PrimIdx = 0; PrimIdx < SelectedPrims.Num(); PrimIdx++)
	{
		const FPrimData& PrimData = SelectedPrims[PrimIdx];

		FTransform PrimTransform = GetPrimTransform(PrimData);

		FVector PrimLocation = PrimTransform.GetLocation();
		PrimLocation += InDrag;
		PrimTransform.SetLocation(PrimLocation);

		SetPrimTransform(PrimData, PrimTransform);
	}

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);
}

void FStaticMeshEditor::RotateSelectedPrims(const FRotator& InRot)
{
	check(StaticMesh->GetBodySetup());
	StaticMesh->GetBodySetup()->InvalidatePhysicsData();

	const FQuat DeltaQ = InRot.Quaternion();

	for (int32 PrimIdx = 0; PrimIdx < SelectedPrims.Num(); PrimIdx++)
	{
		const FPrimData& PrimData = SelectedPrims[PrimIdx];

		FTransform PrimTransform = GetPrimTransform(PrimData);

		FRotator ActorRotWind, ActorRotRem;
		PrimTransform.Rotator().GetWindingAndRemainder(ActorRotWind, ActorRotRem);

		const FQuat ActorQ = ActorRotRem.Quaternion();
		FRotator NewActorRotRem = FRotator(DeltaQ * ActorQ);
		NewActorRotRem.Normalize();
		PrimTransform.SetRotation(NewActorRotRem.Quaternion());

		SetPrimTransform(PrimData, PrimTransform);
	}

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);
}

void FStaticMeshEditor::ScaleSelectedPrims(const FVector& InScale)
{
	check(StaticMesh->GetBodySetup());
	StaticMesh->GetBodySetup()->InvalidatePhysicsData();

	FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

	FVector ModifiedScale = InScale;
	FSnappingUtils::AdjustPercentageBasedScale(ModifiedScale);

	//Multiply in estimated size of the mesh so scaling of sphere, box and sphyl is similar speed to other scaling
	float SimplePrimitiveScaleSpeedFactor = static_cast<float>( StaticMesh->GetBounds().SphereRadius );

	for (int32 PrimIdx = 0; PrimIdx < SelectedPrims.Num(); PrimIdx++)
	{
		const FPrimData& PrimData = SelectedPrims[PrimIdx];

		check(IsPrimValid(PrimData));
		switch (PrimData.PrimType)
		{
		case EAggCollisionShape::Sphere:
			AggGeom->SphereElems[PrimData.PrimIndex].ScaleElem(SimplePrimitiveScaleSpeedFactor * ModifiedScale, MinPrimSize);
			break;
		case EAggCollisionShape::Box:
			AggGeom->BoxElems[PrimData.PrimIndex].ScaleElem(SimplePrimitiveScaleSpeedFactor * ModifiedScale, MinPrimSize);
			break;
		case EAggCollisionShape::Sphyl:
			AggGeom->SphylElems[PrimData.PrimIndex].ScaleElem(SimplePrimitiveScaleSpeedFactor * ModifiedScale, MinPrimSize);
			break;
		case EAggCollisionShape::Convex:
			AggGeom->ConvexElems[PrimData.PrimIndex].ScaleElem(ModifiedScale, MinPrimSize);
			break;
		case EAggCollisionShape::LevelSet:
			{
				// Apply scaling to the centered transform; note that MinPrimSize has no effect for level sets (nor convex hulls)
				FTransform ScaledTransform = AggGeom->LevelSetElems[PrimData.PrimIndex].GetCenteredTransform();
				ScaledTransform.SetScale3D(ScaledTransform.GetScale3D() + ModifiedScale);
				AggGeom->LevelSetElems[PrimData.PrimIndex].SetCenteredTransform(ScaledTransform);
				break;
			}
		}

		StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
	}

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);
}

bool FStaticMeshEditor::CalcSelectedPrimsAABB(FBox &OutBox) const
{
	check(StaticMesh->GetBodySetup());

	FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

	for (int32 PrimIdx = 0; PrimIdx < SelectedPrims.Num(); PrimIdx++)
	{
		const FPrimData& PrimData = SelectedPrims[PrimIdx];

		check(IsPrimValid(PrimData));
		switch (PrimData.PrimType)
		{
		case EAggCollisionShape::Sphere:
			OutBox += AggGeom->SphereElems[PrimData.PrimIndex].CalcAABB(FTransform::Identity, 1.f);
			break;
		case EAggCollisionShape::Box:
			OutBox += AggGeom->BoxElems[PrimData.PrimIndex].CalcAABB(FTransform::Identity, 1.f);
			break;
		case EAggCollisionShape::Sphyl:
			OutBox += AggGeom->SphylElems[PrimData.PrimIndex].CalcAABB(FTransform::Identity, 1.f);
			break;
		case EAggCollisionShape::Convex:
			OutBox += AggGeom->ConvexElems[PrimData.PrimIndex].CalcAABB(FTransform::Identity, FVector(1.f));
			break;
		case EAggCollisionShape::LevelSet:
			OutBox += AggGeom->LevelSetElems[PrimData.PrimIndex].CalcAABB(FTransform::Identity, FVector(1.f));
			break;
		}
	}
	return HasSelectedPrims();
}

bool FStaticMeshEditor::GetLastSelectedPrimTransform(FTransform& OutTransform) const
{
	if (SelectedPrims.Num() > 0)
	{
		check(StaticMesh->GetBodySetup());

		const FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

		const FPrimData& PrimData = SelectedPrims.Last();

		// The SME is not notified of external changes to Simple Collision of the active object, and the UBodySetup
		// does not have any kind of change notification to hook into to do this. So, the SelectedPrims can become
		// invalid if an external change is made. In that case we will just return false here and hope that the 
		// FStaticMeshEditorViewportClient caller will error-handle correctly.
		if (IsPrimValid(PrimData) == false)
		{
			return false;
		}

		switch (PrimData.PrimType)
		{
		case EAggCollisionShape::Sphere:
			OutTransform = AggGeom->SphereElems[PrimData.PrimIndex].GetTransform();
			break;
		case EAggCollisionShape::Box:
			OutTransform = AggGeom->BoxElems[PrimData.PrimIndex].GetTransform();
			break;
		case EAggCollisionShape::Sphyl:
			OutTransform = AggGeom->SphylElems[PrimData.PrimIndex].GetTransform();
			break;
		case EAggCollisionShape::Convex:
			OutTransform = AggGeom->ConvexElems[PrimData.PrimIndex].GetTransform();
			break;
		case EAggCollisionShape::LevelSet:
			OutTransform = AggGeom->LevelSetElems[PrimData.PrimIndex].GetCenteredTransform();
			break;
		}
	}
	return HasSelectedPrims();
}

FTransform FStaticMeshEditor::GetPrimTransform(const FPrimData& InPrimData) const
{
	check(StaticMesh->GetBodySetup());

	const FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

	check(IsPrimValid(InPrimData));
	switch (InPrimData.PrimType)
	{
	case EAggCollisionShape::Sphere:
		return AggGeom->SphereElems[InPrimData.PrimIndex].GetTransform();
	case EAggCollisionShape::Box:
		return AggGeom->BoxElems[InPrimData.PrimIndex].GetTransform();
	case EAggCollisionShape::Sphyl:
		return AggGeom->SphylElems[InPrimData.PrimIndex].GetTransform();
	case EAggCollisionShape::Convex:
		return AggGeom->ConvexElems[InPrimData.PrimIndex].GetTransform();
	case EAggCollisionShape::LevelSet:
		return AggGeom->LevelSetElems[InPrimData.PrimIndex].GetCenteredTransform();
	}
	return FTransform::Identity;
}

void FStaticMeshEditor::SetPrimTransform(const FPrimData& InPrimData, const FTransform& InPrimTransform) const
{
	check(StaticMesh->GetBodySetup());

	FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

	check(IsPrimValid(InPrimData));
	switch (InPrimData.PrimType)
	{
	case EAggCollisionShape::Sphere:
		AggGeom->SphereElems[InPrimData.PrimIndex].SetTransform(InPrimTransform);
		break;
	case EAggCollisionShape::Box:
		AggGeom->BoxElems[InPrimData.PrimIndex].SetTransform(InPrimTransform);
		break;
	case EAggCollisionShape::Sphyl:
		AggGeom->SphylElems[InPrimData.PrimIndex].SetTransform(InPrimTransform);
		break;
	case EAggCollisionShape::Convex:
		AggGeom->ConvexElems[InPrimData.PrimIndex].SetTransform(InPrimTransform);
		break;
	case EAggCollisionShape::LevelSet:
		AggGeom->LevelSetElems[InPrimData.PrimIndex].SetCenteredTransform(InPrimTransform);
		break;
	}

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
}

bool FStaticMeshEditor::OverlapsExistingPrim(const FPrimData& InPrimData) const
{
	check(StaticMesh->GetBodySetup());

	const FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

	// Assume that if the transform of the prim is the same, then it overlaps (FKConvexElem doesn't have an operator==, and no shape takes tolerances into account)
	check(IsPrimValid(InPrimData));
	switch (InPrimData.PrimType)
	{
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem InSphereElem = AggGeom->SphereElems[InPrimData.PrimIndex];
			const FTransform InElemTM = InSphereElem.GetTransform();
			for (int32 i = 0; i < AggGeom->SphereElems.Num(); ++i)
			{
				if( i == InPrimData.PrimIndex )
				{
					continue;
				}

				const FKSphereElem& SphereElem = AggGeom->SphereElems[i];
				const FTransform ElemTM = SphereElem.GetTransform();
				if( InElemTM.Equals(ElemTM) )
				{
					return true;
				}
			}
		}
		break;
	case EAggCollisionShape::Box:
		{
			const FKBoxElem InBoxElem = AggGeom->BoxElems[InPrimData.PrimIndex];
			const FTransform InElemTM = InBoxElem.GetTransform();
			for (int32 i = 0; i < AggGeom->BoxElems.Num(); ++i)
			{
				if( i == InPrimData.PrimIndex )
				{
					continue;
				}

				const FKBoxElem& BoxElem = AggGeom->BoxElems[i];
				const FTransform ElemTM = BoxElem.GetTransform();
				if( InElemTM.Equals(ElemTM) )
				{
					return true;
				}
			}
		}
		break;
	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem InSphylElem = AggGeom->SphylElems[InPrimData.PrimIndex];
			const FTransform InElemTM = InSphylElem.GetTransform();
			for (int32 i = 0; i < AggGeom->SphylElems.Num(); ++i)
			{
				if( i == InPrimData.PrimIndex )
				{
					continue;
				}

				const FKSphylElem& SphylElem = AggGeom->SphylElems[i];
				const FTransform ElemTM = SphylElem.GetTransform();
				if( InElemTM.Equals(ElemTM) )
				{
					return true;
				}
			}
		}
		break;
	case EAggCollisionShape::Convex:
		{
			const FKConvexElem InConvexElem = AggGeom->ConvexElems[InPrimData.PrimIndex];
			const FTransform InElemTM = InConvexElem.GetTransform();
			for (int32 i = 0; i < AggGeom->ConvexElems.Num(); ++i)
			{
				if( i == InPrimData.PrimIndex )
				{
					continue;
				}

				const FKConvexElem& ConvexElem = AggGeom->ConvexElems[i];
				const FTransform ElemTM = ConvexElem.GetTransform();
				if( InElemTM.Equals(ElemTM) )
				{
					return true;
				}
			}
		}
		break;
	case EAggCollisionShape::LevelSet:
		{
			const FKLevelSetElem InLevelSetElem = AggGeom->LevelSetElems[InPrimData.PrimIndex];
			const FTransform InElemTM = InLevelSetElem.GetTransform();
			for (int32 i = 0; i < AggGeom->LevelSetElems.Num(); ++i)
			{
				if (i == InPrimData.PrimIndex)
				{
					continue;
				}

				const FKLevelSetElem& LevelSetElem = AggGeom->LevelSetElems[i];
				const FTransform ElemTM = LevelSetElem.GetTransform();
				if (InElemTM.Equals(ElemTM))
				{
					return true;
				}
			}
		}
		break;
	}

	return false;
}

void FStaticMeshEditor::RefreshTool()
{
	int32 NumLODs = StaticMesh->GetNumLODs();
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		UpdateLODStats(LODIndex);
	}

	OnSelectedLODChangedResetOnRefresh.Clear();
	bool bForceRefresh = true;
	StaticMeshDetailsView->SetObject( StaticMesh, bForceRefresh );

	RefreshViewport();
}

void FStaticMeshEditor::RefreshViewport()
{
	if (GetStaticMeshViewport().IsValid())
	{
		GetStaticMeshViewport()->RefreshViewport();
	}
}

bool FStaticMeshEditor::IsShowSimpleCollisionChecked() const
{
	return GetStaticMeshViewport().IsValid() && GetStaticMeshViewport()->GetViewportClient().IsShowSimpleCollisionChecked();
}

void FStaticMeshEditor::ToggleShowSimpleCollision()
{
	if (GetStaticMeshViewport().IsValid())
	{
		GetStaticMeshViewport()->GetViewportClient().ToggleShowSimpleCollision();
	}
}

void FStaticMeshEditor::GenerateUVChannelComboList(UToolMenu* InMenu)
{
	FUIAction DrawUVsAction;

	FStaticMeshEditorViewportClient& ViewportClient = GetStaticMeshViewport()->GetViewportClient();

	DrawUVsAction.ExecuteAction = FExecuteAction::CreateRaw(&ViewportClient, &FStaticMeshEditorViewportClient::SetDrawUVOverlay, false);

	// Note, the logic is inversed here.  We show the radio button as checked if no uv channels are being shown
	DrawUVsAction.GetActionCheckState = FGetActionCheckState::CreateLambda([&ViewportClient]() {return ViewportClient.IsDrawUVOverlayChecked() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked; });
	
	// Add UV display functions
	{
		FToolMenuSection& Section = InMenu->AddSection("UVDisplayOptions");
		Section.AddMenuEntry(
			"ShowUVSToggle",
			LOCTEXT("ShowUVSToggle", "None"),
			LOCTEXT("ShowUVSToggle_Tooltip", "Toggles display of the static mesh's UVs."),
			FSlateIcon(),
			DrawUVsAction,
			EUserInterfaceActionType::RadioButton
		);

		Section.AddSeparator("ShowUVSToggleSeperator");
		// Fill out the UV channels combo.
		int32 MaxUVChannels = FMath::Max<int32>(GetNumUVChannels(), 1);
		FName UVChannelIDName("UVChannel_ID");
		for (int32 UVChannelID = 0; UVChannelID < MaxUVChannels; ++UVChannelID)
		{
			FUIAction MenuAction;
			MenuAction.ExecuteAction.BindSP(this, &FStaticMeshEditor::SetCurrentViewedUVChannel, UVChannelID);
			MenuAction.GetActionCheckState.BindSP(this, &FStaticMeshEditor::GetUVChannelCheckState, UVChannelID);

			UVChannelIDName.SetNumber(UVChannelID+1);
			Section.AddMenuEntry(
				UVChannelIDName,
				FText::Format(LOCTEXT("UVChannel_ID", "UV Channel {0}"), FText::AsNumber(UVChannelID)),
				FText::Format(LOCTEXT("UVChannel_ID_ToolTip", "Overlay UV Channel {0} on the viewport"), FText::AsNumber(UVChannelID)),
				FSlateIcon(),
				MenuAction,
				EUserInterfaceActionType::RadioButton
			);
		}
	}


	if (TSharedPtr<FStaticMeshEditor> StaticMeshEditor = StaticMeshEditor::GetStaticMeshEditorFromMenuContext(InMenu))
	{
		FToolMenuSection& Section = InMenu->AddSection("UVActionOptions");

		FUIAction MenuAction;
		MenuAction.ExecuteAction.BindSP(this, &FStaticMeshEditor::RemoveCurrentUVChannel);
		MenuAction.CanExecuteAction.BindSP(this, &FStaticMeshEditor::CanRemoveUVChannel);
		Section.AddMenuEntry(
			"Remove_UVChannel",
			LOCTEXT("Remove_UVChannel", "Remove Selected"),
			LOCTEXT("Remove_UVChannel_ToolTip", "Remove currently selected UV channel from the static mesh"),
			FSlateIcon(),
			MenuAction,
			EUserInterfaceActionType::Button
		);
		
	}
}


void FStaticMeshEditor::UpdateLODStats(int32 CurrentLOD)
{
	NumTriangles[CurrentLOD] = 0; //-V781
	NumVertices[CurrentLOD] = 0;
	NumUVChannels[CurrentLOD] = 0;
	int32 NumLODLevels = 0;

	if( StaticMesh->GetRenderData())
	{
		NumLODLevels = StaticMesh->GetRenderData()->LODResources.Num();
		if (CurrentLOD >= 0 && CurrentLOD < NumLODLevels)
		{
			FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[CurrentLOD];
			NumTriangles[CurrentLOD] = LODModel.GetNumTriangles();
			NumVertices[CurrentLOD] = LODModel.GetNumVertices();
			NumUVChannels[CurrentLOD] = LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		}
	}
}

void FStaticMeshEditor::ComboBoxSelectionChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	GetStaticMeshViewport()->RefreshViewport();
}

void FStaticMeshEditor::HandleReimportMesh(const FReimportParameters ReimportParameters)
{
	// Reimport the asset
	if (StaticMesh)
	{
		constexpr bool bAskForNewFileIfMissingTrue = true;
		constexpr bool bShowNotificationTrue = true;
		const FString PreferredReimportFileEmpty = TEXT("");
		constexpr FReimportHandler* SpecifiedReimportHandlerNull = nullptr;
		constexpr int32 SourceFileIndex = INDEX_NONE;
		constexpr bool bAutomatedFalse = false;
		FReimportManager::Instance()->ReimportAsync(StaticMesh
			, bAskForNewFileIfMissingTrue
			, bShowNotificationTrue
			, PreferredReimportFileEmpty
			, SpecifiedReimportHandlerNull
			, SourceFileIndex
			, ReimportParameters.bWithNewFile
			, bAutomatedFalse
			, ReimportParameters.bWithDialog);
	}
}

void FStaticMeshEditor::HandleReimportAllMesh(const FReimportParameters ReimportParameters)
{
	// Reimport the asset
	if (StaticMesh)
	{
		constexpr bool bAskForNewFileIfMissingTrue = true;
		constexpr bool bShowNotificationTrue = true;
		const FString PreferredReimportFileEmpty = TEXT("");
		constexpr FReimportHandler* SpecifiedReimportHandlerNull = nullptr;
		constexpr int32 SourceFileIndex = INDEX_NONE;
		constexpr bool bAutomatedFalse = false;
		//Reimport base LOD, generated mesh will be rebuild here, the static mesh is always using the base mesh to reduce LOD
		if (FReimportManager::Instance()->Reimport(StaticMesh
			, bAskForNewFileIfMissingTrue
			, bShowNotificationTrue
			, PreferredReimportFileEmpty
			, SpecifiedReimportHandlerNull
			, SourceFileIndex
			, ReimportParameters.bWithNewFile
			, bAutomatedFalse
			, ReimportParameters.bWithDialog))
		{
			//Reimport all custom LODs
			for (int32 LodIndex = 1; LodIndex < StaticMesh->GetNumLODs(); ++LodIndex)
			{
				//Skip LOD import in the same file as the base mesh, they are already re-import
				if (StaticMesh->GetSourceModel(LodIndex).bImportWithBaseMesh)
				{
					continue;
				}

				bool bHasBeenSimplified = !StaticMesh->IsMeshDescriptionValid(LodIndex) || StaticMesh->IsReductionActive(LodIndex);
				if (!bHasBeenSimplified)
				{
					FbxMeshUtils::ImportMeshLODDialog(StaticMesh, LodIndex);
				}
			}
		}
	}
}

int32 FStaticMeshEditor::GetCurrentUVChannel()
{
	return FMath::Min(CurrentViewedUVChannel, GetNumUVChannels());
}

int32 FStaticMeshEditor::GetCurrentLODLevel()
{
	if (GetStaticMeshComponent())
	{
		return GetStaticMeshComponent()->ForcedLodModel;
	}
	return 0;
}

int32 FStaticMeshEditor::GetCurrentLODIndex()
{
	int32 Index = GetCurrentLODLevel();

	return Index == 0? 0 : Index - 1;
}

int32 FStaticMeshEditor::GetCustomData(const int32 Key) const
{
	if (!CustomEditorData.Contains(Key))
	{
		return INDEX_NONE;
	}
	return CustomEditorData[Key];
}

void FStaticMeshEditor::SetCustomData(const int32 Key, const int32 CustomData)
{
	CustomEditorData.FindOrAdd(Key) = CustomData;
}

void FStaticMeshEditor::GenerateKDop(const FVector* Directions, uint32 NumDirections)
{
	TArray<FVector>	DirArray;
	for(uint32 DirectionIndex = 0;DirectionIndex < NumDirections;DirectionIndex++)
	{
		DirArray.Add(Directions[DirectionIndex]);
	}

	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_GenerateKDop", "Create Convex Collision"));
	const int32 PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
	if (PrimIndex != INDEX_NONE)
	{
		StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
	}
	GEditor->EndTransaction();
	if (PrimIndex != INDEX_NONE)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Collision"), TEXT("Type"), TEXT("KDop Collision"));
		}
		const FPrimData PrimData = FPrimData(EAggCollisionShape::Convex, PrimIndex);
		ClearSelectedPrims();
		AddSelectedPrim(PrimData, true);
		// Don't 'nudge' KDop prims, as they are fitted specifically around the geometry
	}

	GetStaticMeshViewport()->RefreshViewport();
}

void FStaticMeshEditor::OnCollisionBox()
{
	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_OnCollisionBox", "Create Box Collision"));
	const int32 PrimIndex = GenerateBoxAsSimpleCollision(StaticMesh);
	if (PrimIndex != INDEX_NONE)
	{
		StaticMesh->GetBodySetup()->AggGeom.BoxElems[PrimIndex].bIsGenerated = true;
	}
	GEditor->EndTransaction();
	if (PrimIndex != INDEX_NONE)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Collision"), TEXT("Type"), TEXT("Box Collision"));
		}
		const FPrimData PrimData = FPrimData(EAggCollisionShape::Box, PrimIndex);
		ClearSelectedPrims();
		AddSelectedPrim(PrimData, true);
		while( OverlapsExistingPrim(PrimData) )
		{
			TranslateSelectedPrims(OverlapNudge);
		}
	}

	GetStaticMeshViewport()->RefreshViewport();
}

void FStaticMeshEditor::OnCollisionSphere()
{
	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_OnCollisionSphere", "Create Sphere Collision"));
	const int32 PrimIndex = GenerateSphereAsSimpleCollision(StaticMesh);
	if (PrimIndex != INDEX_NONE)
	{
		StaticMesh->GetBodySetup()->AggGeom.SphereElems[PrimIndex].bIsGenerated = true;
	}
	GEditor->EndTransaction();
	if (PrimIndex != INDEX_NONE)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Collision"), TEXT("Type"), TEXT("Sphere Collision"));
		}
		const FPrimData PrimData = FPrimData(EAggCollisionShape::Sphere, PrimIndex);
		ClearSelectedPrims();
		AddSelectedPrim(PrimData, true);
		while( OverlapsExistingPrim(PrimData) )
		{
			TranslateSelectedPrims(OverlapNudge);
		}
	}

	GetStaticMeshViewport()->RefreshViewport();
}

void FStaticMeshEditor::OnCollisionSphyl()
{
	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_OnCollisionSphyl", "Create Capsule Collision"));
	const int32 PrimIndex = GenerateSphylAsSimpleCollision(StaticMesh);
	if (PrimIndex != INDEX_NONE)
	{
		StaticMesh->GetBodySetup()->AggGeom.SphylElems[PrimIndex].bIsGenerated = true;
	}
	GEditor->EndTransaction();
	if (PrimIndex != INDEX_NONE)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Collision"), TEXT("Type"), TEXT("Capsule Collision"));
		}
		const FPrimData PrimData = FPrimData(EAggCollisionShape::Sphyl, PrimIndex);
		ClearSelectedPrims();
		AddSelectedPrim(PrimData, true);
		while( OverlapsExistingPrim(PrimData) )
		{
			TranslateSelectedPrims(OverlapNudge);
		}
	}

	GetStaticMeshViewport()->RefreshViewport();
}

void FStaticMeshEditor::OnRemoveCollision(void)
{
	UBodySetup* BS = StaticMesh->GetBodySetup();
	check(BS != NULL && BS->AggGeom.GetElementCount() > 0);

	ClearSelectedPrims();

	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_RemoveCollision", "Remove Collision"));
	StaticMesh->GetBodySetup()->Modify();

	StaticMesh->GetBodySetup()->RemoveSimpleCollision();

	GEditor->EndTransaction();

	// refresh collision change back to staticmesh components
	RefreshCollisionChange(*StaticMesh);

	// Mark staticmesh as dirty, to help make sure it gets saved.
	StaticMesh->MarkPackageDirty();

	// Update views/property windows
	GetStaticMeshViewport()->RefreshViewport();

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
}

bool FStaticMeshEditor::CanRemoveCollision()
{
	UBodySetup* BS = StaticMesh->GetBodySetup();
	return (BS != NULL && BS->AggGeom.GetElementCount() > 0);
}

/** Util for adding vertex to an array if it is not already present. */
static void AddVertexIfNotPresent(TArray<FVector3f>& Vertices, const FVector3f& NewVertex)
{
	bool bIsPresent = false;

	for(int32 i=0; i<Vertices.Num(); i++)
	{
		float diffSqr = (NewVertex - Vertices[i]).SizeSquared();
		if(diffSqr < 0.01f * 0.01f)
		{
			bIsPresent = 1;
			break;
		}
	}

	if(!bIsPresent)
	{
		Vertices.Add(NewVertex);
	}
}

void FStaticMeshEditor::OnConvertBoxToConvexCollision()
{
	// If we have a collision model for this staticmesh, ask if we want to replace it.
	if (StaticMesh->GetBodySetup())
	{
		int32 ShouldReplace = FMessageDialog::Open( EAppMsgType::YesNo, LOCTEXT("ConvertBoxCollisionPrompt", "Are you sure you want to convert all box collision?") );
		if (ShouldReplace == EAppReturnType::Yes)
		{
			UBodySetup* BodySetup = StaticMesh->GetBodySetup();

			int32 NumBoxElems = BodySetup->AggGeom.BoxElems.Num();
			if (NumBoxElems > 0)
			{
				ClearSelectedPrims();

				// Make sure rendering is done - so we are not changing data being used by collision drawing.
				FlushRenderingCommands();

				FKConvexElem* NewConvexColl = NULL;

				//For each box elem, calculate the new convex collision representation
				//Stored in a temp array so we can undo on failure.
				TArray<FKConvexElem> TempArray;

				for (int32 i=0; i<NumBoxElems; i++)
				{
					const FKBoxElem& BoxColl = BodySetup->AggGeom.BoxElems[i];

					//Create a new convex collision element
					NewConvexColl = new(TempArray) FKConvexElem();
					NewConvexColl->ConvexFromBoxElem(BoxColl);
				}

				//Clear the cache (PIE may have created some data), create new GUID
				BodySetup->InvalidatePhysicsData();

				//Copy the new data into the static mesh
				BodySetup->AggGeom.ConvexElems.Append(TempArray);

				//Clear out what we just replaced
				BodySetup->AggGeom.BoxElems.Empty();

				BodySetup->CreatePhysicsMeshes();

				// Select the new prims
				FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;
				for (int32 i = 0; i < NumBoxElems; ++i)
				{
					AddSelectedPrim(FPrimData(EAggCollisionShape::Convex, (AggGeom->ConvexElems.Num() - (i+1))), false);
				}

				RefreshCollisionChange(*StaticMesh);
				// Mark static mesh as dirty, to help make sure it gets saved.
				StaticMesh->MarkPackageDirty();

				// Update views/property windows
				GetStaticMeshViewport()->RefreshViewport();

				StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
			}
		}
	}
}

void FStaticMeshEditor::OnCopyCollisionFromSelectedStaticMesh()
{
	UStaticMesh* SelectedMesh = GetFirstSelectedStaticMeshInContentBrowser();
	check(SelectedMesh && SelectedMesh != StaticMesh && SelectedMesh->GetBodySetup());

	UBodySetup* BodySetup = StaticMesh->GetBodySetup();

	ClearSelectedPrims();

	// Make sure rendering is done - so we are not changing data being used by collision drawing.
	FlushRenderingCommands();

	GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_CopyCollisionFromSelectedStaticMesh", "Copy Collision from Selected Static Mesh"));
	BodySetup->Modify();

	// Copy body properties from
	BodySetup->CopyBodyPropertiesFrom(SelectedMesh->GetBodySetup());

	// Enable collision, if not already
	if( !GetStaticMeshViewport()->GetViewportClient().IsShowSimpleCollisionChecked() )
	{
		GetStaticMeshViewport()->GetViewportClient().ToggleShowSimpleCollision();
	}

	// Invalidate physics data and create new meshes
	BodySetup->InvalidatePhysicsData();
	BodySetup->CreatePhysicsMeshes();

	GEditor->EndTransaction();

	RefreshCollisionChange(*StaticMesh);
	// Mark static mesh as dirty, to help make sure it gets saved.
	StaticMesh->MarkPackageDirty();

	// Redraw level editor viewports, in case the asset's collision is visible in a viewport and the viewport isn't set to realtime.
	// Note: This could be more intelligent and only trigger a redraw if the asset is referenced in the world.
	GUnrealEd->RedrawLevelEditingViewports();

	// Update views/property windows
	GetStaticMeshViewport()->RefreshViewport();

	StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
}

bool FStaticMeshEditor::CanCopyCollisionFromSelectedStaticMesh() const
{
	bool CanCopy = false;

	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);
	if(SelectedAssets.Num() == 1)
	{
		FAssetData& Asset = SelectedAssets[0];
		if(Asset.GetClass() == UStaticMesh::StaticClass())
		{
			UStaticMesh* SelectedMesh = Cast<UStaticMesh>(Asset.GetAsset());
			if(SelectedMesh && SelectedMesh != StaticMesh && SelectedMesh->GetBodySetup())
			{
				CanCopy = true;
			}
		}
	}

	return CanCopy;
}

UStaticMesh* FStaticMeshEditor::GetFirstSelectedStaticMeshInContentBrowser() const
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for(auto& Asset : SelectedAssets)
	{
		UStaticMesh* SelectedMesh = Cast<UStaticMesh>(Asset.GetAsset());
		if(SelectedMesh)
		{
			return SelectedMesh;
		}
	}

	return NULL;
}

void FStaticMeshEditor::SetEditorMesh(UStaticMesh* InStaticMesh, bool bResetCamera/*=true*/)
{
	ClearSelectedPrims();

	if (StaticMesh)
	{
		StaticMesh->GetOnMeshChanged().RemoveAll(this);
	}

	StaticMesh = InStaticMesh;

	//Init stat arrays.
	const int32 ArraySize = MAX_STATIC_MESH_LODS;
	NumVertices.Empty(ArraySize);
	NumVertices.AddZeroed(ArraySize);
	NumTriangles.Empty(ArraySize);
	NumTriangles.AddZeroed(ArraySize);
	NumUVChannels.Empty(ArraySize);
	NumUVChannels.AddZeroed(ArraySize);

	if(StaticMesh)
	{
		StaticMesh->GetOnMeshChanged().AddRaw(this, &FStaticMeshEditor::OnMeshChanged);

		int32 NumLODs = StaticMesh->GetNumLODs();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			UpdateLODStats(LODIndex);
		}
	}

	// Set the details view.
	StaticMeshDetailsView->SetObject(StaticMesh);

	if (SStaticMeshEditorViewport* StaticMeshViewport = GetStaticMeshViewport().Get())
	{
		StaticMeshViewport->UpdatePreviewMesh(StaticMesh, bResetCamera);
		StaticMeshViewport->RefreshViewport();
		if (EditorModeManager)
		{
			// update the selection
			if (USelection* ComponentSet = EditorModeManager->GetSelectedComponents())
			{
				ComponentSet->BeginBatchSelectOperation();
				ComponentSet->DeselectAll();
				ComponentSet->Select(StaticMeshViewport->GetStaticMeshComponent(), true);
				ComponentSet->EndBatchSelectOperation();
			}
		}
	}
}

void FStaticMeshEditor::OnChangeMesh()
{
	UStaticMesh* SelectedMesh = GetFirstSelectedStaticMeshInContentBrowser();
	check(SelectedMesh != NULL && SelectedMesh != StaticMesh);

	RemoveEditingObject(StaticMesh);
	AddEditingObject(SelectedMesh);

	SetEditorMesh(SelectedMesh);

	// Clear selections made on previous mesh
	ClearSelectedPrims();
	GetSelectedEdges().Empty();

	if(SocketManager.IsValid())
	{
		SocketManager->UpdateStaticMesh();
	}
}

bool FStaticMeshEditor::CanChangeMesh() const
{
	bool CanChange = false;

	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);
	if(SelectedAssets.Num() == 1)
	{
		FAssetData& Asset = SelectedAssets[0];
		if(Asset.GetClass() == UStaticMesh::StaticClass())
		{
			UStaticMesh* SelectedMesh = Cast<UStaticMesh>(Asset.GetAsset());
			if(SelectedMesh && SelectedMesh != StaticMesh)
			{
				CanChange = true;
			}
		}
	}

	return CanChange;
}

void FStaticMeshEditor::OnSaveGeneratedLODs()
{
	if (StaticMesh)
	{
		StaticMesh->GenerateLodsInPackage();

		// Update editor UI as we modified LOD groups
		auto Selected = StaticMeshDetailsView->GetSelectedObjects();
		StaticMeshDetailsView->SetObjects(Selected, true);

		// Update screen
		GetStaticMeshViewport()->RefreshViewport();
	}
}

void FStaticMeshEditor::DoDecomp(uint32 InHullCount, int32 InMaxHullVerts, uint32 InHullPrecision)
{
	// Check we have a selected StaticMesh
	if(StaticMesh && StaticMesh->GetRenderData())
	{
		FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[0];

		// Start a busy cursor so the user has feedback while waiting
		const FScopedBusyCursor BusyCursor;

		// Make vertex buffer
		int32 NumVerts = LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
		TArray<FVector3f> Verts;
		Verts.SetNumUninitialized(NumVerts);
		for(int32 i=0; i<NumVerts; i++)
		{
			const FVector3f& Vert = LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(i);
			Verts[i] = Vert;
		}

		// Grab all indices
		TArray<uint32> AllIndices;
		LODModel.IndexBuffer.GetCopy(AllIndices);

		// Only copy indices that have collision enabled
		TArray<uint32> CollidingIndices;
		for(const FStaticMeshSection& Section : LODModel.Sections)
		{
			if(Section.bEnableCollision)
			{
				for (uint32 IndexIdx = Section.FirstIndex; IndexIdx < Section.FirstIndex + (Section.NumTriangles * 3); IndexIdx++)
				{
					CollidingIndices.Add(AllIndices[IndexIdx]);
				}
			}
		}

		ClearSelectedPrims();

		// Make sure rendering is done - so we are not changing data being used by collision drawing.
		FlushRenderingCommands();

		// Get the BodySetup we are going to put the collision into
		UBodySetup* bs = StaticMesh->GetBodySetup();
		if(bs)
		{
			bs->RemoveSimpleCollision();
		}
		else
		{
			// Otherwise, create one here.
			StaticMesh->CreateBodySetup();
			bs = StaticMesh->GetBodySetup();
		}

		// Run actual util to do the work (if we have some valid input)
		if(Verts.Num() >= 3 && CollidingIndices.Num() >= 3)
		{
#if USE_ASYNC_DECOMP
			// If there is currently a decomposition already in progress we release it.
			if (DecomposeMeshToHullsAsync)
			{
				DecomposeMeshToHullsAsync->Release();
			}
			// Begin the convex decomposition process asynchronously
			DecomposeMeshToHullsAsync = CreateIDecomposeMeshToHullAsync();
			DecomposeMeshToHullsAsync->DecomposeMeshToHullsAsyncBegin(bs, MoveTemp(Verts), MoveTemp(CollidingIndices), InHullCount, InMaxHullVerts, InHullPrecision);
#else
			DecomposeMeshToHulls(bs, Verts, CollidingIndices, InHullCount, InMaxHullVerts, InHullPrecision);
#endif
		}

		// Enable collision, if not already
		if( !GetStaticMeshViewport()->GetViewportClient().IsShowSimpleCollisionChecked() )
		{
			GetStaticMeshViewport()->GetViewportClient().ToggleShowSimpleCollision();
		}

		// refresh collision change back to staticmesh components
		RefreshCollisionChange(*StaticMesh);

		// Mark mesh as dirty
		StaticMesh->MarkPackageDirty();

		// Update screen.
		GetStaticMeshViewport()->RefreshViewport();

		StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
	}
}

TSet< int32 >& FStaticMeshEditor::GetSelectedEdges()
{
	return GetStaticMeshViewport()->GetSelectedEdges();
}

int32 FStaticMeshEditor::GetNumTriangles( int32 LODLevel ) const
{
	return NumTriangles.IsValidIndex(LODLevel) ? NumTriangles[LODLevel] : 0;
}

int32 FStaticMeshEditor::GetNumVertices( int32 LODLevel ) const
{
	return NumVertices.IsValidIndex(LODLevel) ? NumVertices[LODLevel] : 0;
}

int32 FStaticMeshEditor::GetNumUVChannels( int32 LODLevel ) const
{
	return NumUVChannels.IsValidIndex(LODLevel) ? NumUVChannels[LODLevel] : 0;
}

void FStaticMeshEditor::DeleteSelected()
{
	if (HasSelectedSockets())
	{
		DeleteSelectedSockets();
	}

	if (HasSelectedPrims())
	{
		DeleteSelectedPrims();
	}
}

bool FStaticMeshEditor::CanDeleteSelected() const
{
	return GetOpenMethod() != EAssetOpenMethod::View && (HasSelectedSockets() || HasSelectedPrims());
}

void FStaticMeshEditor::DeleteSelectedSockets()
{
	check(SocketManager.IsValid());

	SocketManager->DeleteSelectedSockets();
}

void FStaticMeshEditor::DeleteSelectedPrims()
{
	if (SelectedPrims.Num() > 0)
	{
		// Sort the selected prims by PrimIndex so when we're deleting them we don't mess up other prims indicies
		struct FCompareFPrimDataPrimIndex
		{
			FORCEINLINE bool operator()(const FPrimData& A, const FPrimData& B) const
			{
				return A.PrimIndex < B.PrimIndex;
			}
		};
		SelectedPrims.Sort(FCompareFPrimDataPrimIndex());

		check(StaticMesh->GetBodySetup());

		FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

		GEditor->BeginTransaction(LOCTEXT("FStaticMeshEditor_DeleteSelectedPrims", "Delete Collision"));
		StaticMesh->GetBodySetup()->Modify();

		for (int32 PrimIdx = SelectedPrims.Num() - 1; PrimIdx >= 0; PrimIdx--)
		{
			const FPrimData& PrimData = SelectedPrims[PrimIdx];

			check(IsPrimValid(PrimData));
			switch (PrimData.PrimType)
			{
			case EAggCollisionShape::Sphere:
				AggGeom->SphereElems.RemoveAt(PrimData.PrimIndex);
				break;
			case EAggCollisionShape::Box:
				AggGeom->BoxElems.RemoveAt(PrimData.PrimIndex);
				break;
			case EAggCollisionShape::Sphyl:
				AggGeom->SphylElems.RemoveAt(PrimData.PrimIndex);
				break;
			case EAggCollisionShape::Convex:
				AggGeom->ConvexElems.RemoveAt(PrimData.PrimIndex);
				break;
			case EAggCollisionShape::LevelSet:
				AggGeom->LevelSetElems.RemoveAt(PrimData.PrimIndex);
				break;
			}
		}

		GEditor->EndTransaction();

		ClearSelectedPrims();

		// Make sure rendering is done - so we are not changing data being used by collision drawing.
		FlushRenderingCommands();

		// Make sure to invalidate cooked data
		StaticMesh->GetBodySetup()->InvalidatePhysicsData();

		// refresh collision change back to staticmesh components
		RefreshCollisionChange(*StaticMesh);

		// Mark staticmesh as dirty, to help make sure it gets saved.
		StaticMesh->MarkPackageDirty();

		// Update views/property windows
		GetStaticMeshViewport()->RefreshViewport();

		StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization
	}
}

void FStaticMeshEditor::DuplicateSelected()
{
	DuplicateSelectedSockets();

	const FVector InitialOffset(20.f);
	DuplicateSelectedPrims(&InitialOffset);
}

bool FStaticMeshEditor::CanDuplicateSelected() const
{
	return GetOpenMethod() != EAssetOpenMethod::View && (HasSelectedSockets() || HasSelectedPrims());
}

void FStaticMeshEditor::CopySelected()
{
	CopySelectedPrims();
}

bool FStaticMeshEditor::CanCopySelected() const
{
	return HasSelectedPrims();
}

void FStaticMeshEditor::PasteCopied()
{
	PasteCopiedPrims();
}

bool FStaticMeshEditor::CanPasteCopied() const
{
	if(GetOpenMethod() == EAssetOpenMethod::View)
	{
		return false;
	}
	
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	FBodySetupObjectTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

bool FStaticMeshEditor::CanRenameSelected() const
{
	return GetOpenMethod() != EAssetOpenMethod::View && (HasSelectedSockets());
}

void FStaticMeshEditor::ExecuteFindInExplorer()
{
	if ( ensure(StaticMesh->GetAssetImportData()) )
	{
		const FString SourceFilePath = StaticMesh->GetAssetImportData()->GetFirstFilename();
		if ( SourceFilePath.Len() && IFileManager::Get().FileSize( *SourceFilePath ) != INDEX_NONE )
		{
			FPlatformProcess::ExploreFolder( *FPaths::GetPath(SourceFilePath) );
		}
	}
}

bool FStaticMeshEditor::CanExecuteSourceCommands() const
{
	if ( !StaticMesh->GetAssetImportData() )
	{
		return false;
	}

	const FString& SourceFilePath = StaticMesh->GetAssetImportData()->GetFirstFilename();

	return SourceFilePath.Len() && IFileManager::Get().FileSize(*SourceFilePath) != INDEX_NONE;
}

void FStaticMeshEditor::OnObjectReimported(UObject* InObject)
{
	// Make sure we are using the object that is being reimported, otherwise a lot of needless work could occur.
	if(StaticMesh == InObject)
	{
		//When we re-import we want to avoid moving the camera in the staticmesh editor
		bool bResetCamera = false;
		SetEditorMesh(Cast<UStaticMesh>(InObject), bResetCamera);

		if (SocketManager.IsValid())
		{
			SocketManager->UpdateStaticMesh();
		}
	}
}

EViewModeIndex FStaticMeshEditor::GetViewMode() const
{
	if (GetStaticMeshViewport().IsValid())
	{
		const FStaticMeshEditorViewportClient& ViewportClient = GetStaticMeshViewport()->GetViewportClient();
		return ViewportClient.GetViewMode();
	}
	else
	{
		return VMI_Unknown;
	}
}

FEditorViewportClient& FStaticMeshEditor::GetViewportClient()
{
	return GetStaticMeshViewport()->GetViewportClient();
}

void FStaticMeshEditor::OnConvexDecomposition()
{
	TabManager->TryInvokeTab(CollisionTabId);
}

bool FStaticMeshEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	bool bAllowClose = true;
	// If we are in read only mode, don't show the save prompt
	if (GetOpenMethod() != EAssetOpenMethod::View && InCloseReason != EAssetEditorCloseReason::AssetForceDeleted && StaticMeshDetails.IsValid() && StaticMeshDetails.Pin()->IsApplyNeeded())
	{
		// find out the user wants to do with this dirty material
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(
			EAppMsgType::YesNoCancel,
			FText::Format( LOCTEXT("ShouldApplyLODChanges", "Would you like to apply level of detail changes to {0}?\n\n(No will lose all changes!)"), FText::FromString( StaticMesh->GetName() ) )
		);

		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			StaticMeshDetails.Pin()->ApplyChanges();
			bAllowClose = true;
			break;
		case EAppReturnType::No:
			// Do nothing, changes will be abandoned.
			bAllowClose = true;
			break;
		case EAppReturnType::Cancel:
			// Don't exit.
			bAllowClose = false;
			break;
		}
	}

	bAllowClose &= GetEditorModeManager().OnRequestClose();

	// Give any active modes a chance to shutdown while the toolkit host is still alive
	if (bAllowClose)
	{
		GetEditorModeManager().ActivateDefaultMode();
	}

	return bAllowClose;
}

void FStaticMeshEditor::SetupReadOnlyMenuProfiles(FReadOnlyAssetEditorCustomization& OutReadOnlyCustomization)
{
	FName ReadOnlyOwnerName("StaticMeshEditorReadOnly");

	// The combo button to show UVs is fine to be available in read only mode
	OutReadOnlyCustomization.ToolbarPermissionList.AddAllowListItem(ReadOnlyOwnerName, "UVToolbar");

	// Hide the command to bake materials in the "Asset" menu in read only mode
	FNamePermissionList& AssetMenuPermissionList = OutReadOnlyCustomization.MainMenuSubmenuPermissionLists.FindOrAdd("Asset");
	AssetMenuPermissionList.AddDenyListItem(ReadOnlyOwnerName, FStaticMeshEditorCommands::Get().BakeMaterials->GetCommandName());

	// Hide the command to edit sockets in the "Edit" menu in read only mode
	FNamePermissionList& EditMenuPermissionList = OutReadOnlyCustomization.MainMenuSubmenuPermissionLists.FindOrAdd("Edit");
	EditMenuPermissionList.AddDenyListItem(ReadOnlyOwnerName, "DeleteSocket");
	EditMenuPermissionList.AddDenyListItem(ReadOnlyOwnerName, "DuplicateSocket");


}

void FStaticMeshEditor::RegisterOnPostUndo( const FOnPostUndo& Delegate )
{
	OnPostUndo.Add( Delegate );
}

void FStaticMeshEditor::UnregisterOnPostUndo( SWidget* Widget )
{
	OnPostUndo.RemoveAll( Widget );
}

void FStaticMeshEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged )
{
	if(StaticMesh && StaticMesh->GetBodySetup())
	{
		StaticMesh->GetBodySetup()->CreatePhysicsMeshes();
		RemoveInvalidPrims();

		if (UStaticMesh::GetLODGroupMemberName() == PropertyChangedEvent.GetPropertyName())
		{
			RefreshTool();
		}
		else if (PropertyChangedEvent.GetPropertyName() == TEXT("CollisionResponses"))
		{
			for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
			{
				UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*Iter);
				if (StaticMeshComponent->GetStaticMesh() == StaticMesh)
				{
					StaticMeshComponent->UpdateCollisionFromStaticMesh();
					StaticMeshComponent->MarkRenderTransformDirty();
				}
			}
		}
	}
	else
	{
		RemoveInvalidPrims();
	}
}

void FStaticMeshEditor::UndoAction()
{
	GEditor->UndoTransaction();
}

void FStaticMeshEditor::RedoAction()
{
	GEditor->RedoTransaction();
}

void FStaticMeshEditor::PostUndo( bool bSuccess )
{
	RemoveInvalidPrims();
	RefreshTool();

	OnPostUndo.Broadcast();
}

void FStaticMeshEditor::PostRedo( bool bSuccess )
{
	RemoveInvalidPrims();
	RefreshTool();

	OnPostUndo.Broadcast();
}

void FStaticMeshEditor::OnMeshChanged()
{
	GetStaticMeshViewport()->GetViewportClient().OnMeshChanged();
}

void FStaticMeshEditor::OnSocketSelectionChanged()
{
	if (HasSelectedSockets())
	{
		ClearSelectedPrims();
		GetStaticMeshViewport()->GetViewportClient().OnSocketSelectionChanged(GetSelectedSockets()[0]);
	}
	else
	{
		GetStaticMeshViewport()->GetViewportClient().OnSocketSelectionChanged(nullptr);
	}
}

void FStaticMeshEditor::OnPostReimport(UObject* InObject, bool bSuccess)
{
	// Ignore if this is regarding a different object
	if ( InObject != StaticMesh )
	{
		return;
	}

	if (bSuccess)
	{
		RefreshTool();
	}
}

void FStaticMeshEditor::SetCurrentViewedUVChannel(int32 InNewUVChannel)
{
	CurrentViewedUVChannel = FMath::Clamp(InNewUVChannel, 0, GetNumUVChannels());
	GetStaticMeshViewport()->GetViewportClient().SetDrawUVOverlay(true);
}

ECheckBoxState FStaticMeshEditor::GetUVChannelCheckState(int32 TestUVChannel) const
{
	return CurrentViewedUVChannel == TestUVChannel && GetStaticMeshViewport()->GetViewportClient().IsDrawUVOverlayChecked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FStaticMeshEditor::Tick(float DeltaTime)
{
#if USE_ASYNC_DECOMP
	/** If we have an active convex decomposition task running, we check to see if is completed and, if so, release the interface */
	if (DecomposeMeshToHullsAsync)
	{
		if (DecomposeMeshToHullsAsync->IsComplete())
		{
			DecomposeMeshToHullsAsync->Release();
			DecomposeMeshToHullsAsync = nullptr;
			GConvexDecompositionNotificationState->IsActive = false;
		}
		else if (GConvexDecompositionNotificationState)
		{
			GConvexDecompositionNotificationState->IsActive = true;
			GConvexDecompositionNotificationState->Status = DecomposeMeshToHullsAsync->GetCurrentStatus();
		}
	}
#endif
}

TStatId FStaticMeshEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FStaticMeshEditor, STATGROUP_TaskGraphTasks);
}

void FStaticMeshEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	TSharedPtr<SStaticMeshEditorViewport> Viewport = GetStaticMeshViewport();

	if (Viewport.IsValid() && Viewport->GetViewportOverlay().IsValid())
	{
		Viewport->GetViewportOverlay()->AddSlot(ZOrder)
		[
			InOverlaidWidget
		];
	}
}

void FStaticMeshEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
{
	TSharedPtr<SStaticMeshEditorViewport> Viewport = GetStaticMeshViewport();

	if (Viewport.IsValid() && Viewport->GetViewportOverlay().IsValid())
	{
		Viewport->GetViewportOverlay()->RemoveSlot(InViewportOverlayWidget);
	}
}


void FStaticMeshEditor::CreateEditorModeManager()
{
	//
	// This function doesn't actually create a new manager -- it assigns StaticMeshEditorViewport->GetViewportClient().GetModeTools() 
	// to this->EditorModeManager. This is because these two pointers should refer to the same mode manager object, and currently
	// the ViewPortClient's ModeTools object is created first.
	// 
	// This function also:
	// - sets the manager's PreviewScene to be StaticMeshEditorViewport->GetPreviewScene()
	// - adds StaticMeshEditorViewport->GetStaticMeshComponent() to the manager's ComponentSet (i.e. selected mesh components)
	// 
	
	TSharedPtr<FAssetEditorModeManager> NewManager = MakeShared<FAssetEditorModeManager>();

	TSharedPtr<SStaticMeshEditorViewport> StaticMeshEditorViewport = GetStaticMeshViewport();
	if (StaticMeshEditorViewport.IsValid())
	{
		TSharedPtr<FEditorModeTools> SharedModeTools = StaticMeshEditorViewport->GetViewportClient().GetModeTools()->AsShared();
		NewManager = StaticCastSharedPtr<FAssetEditorModeManager>(SharedModeTools);
		check(NewManager.IsValid());

		TSharedRef<FAdvancedPreviewScene> PreviewScene = StaticMeshEditorViewport->GetPreviewScene();
		NewManager->SetPreviewScene(&PreviewScene.Get());

		UStaticMeshComponent* const Component = StaticMeshEditorViewport->GetStaticMeshComponent();

		// Copied from FPersonaEditorModeManager::SetPreviewScene(FPreviewScene * NewPreviewScene)
		USelection* ComponentSet = NewManager->GetSelectedComponents();
		ComponentSet->BeginBatchSelectOperation();
		ComponentSet->DeselectAll();
		ComponentSet->Select(Component, true);
		ComponentSet->EndBatchSelectOperation();
	}

	EditorModeManager = NewManager;
}

bool FStaticMeshEditor::CanRemoveUVChannel()
{
	// Can remove UV channel if there's one that is currently being selected and displayed, 
	// and the current LOD has more than one UV channel
	return GetStaticMeshViewport()->GetViewportClient().IsDrawUVOverlayChecked() && 
		StaticMesh->GetNumUVChannels(GetCurrentLODIndex()) > 1;
}

void FStaticMeshEditor::ToggleShowNormals()
{
	bDrawNormals = !bDrawNormals;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowNormalsFunc = 
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient  = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowNormals(bDrawNormals);
	};

  	ViewportTabContent->PerformActionOnViewports(ToggleShowNormalsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawNormals"), bDrawNormals ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowNormalsChecked() const
{
	return bDrawNormals;
}

void FStaticMeshEditor::ToggleShowTangents()
{
	bDrawTangents = !bDrawTangents;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowTangentsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowTangents(bDrawTangents);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowTangentsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawTangents"), bDrawTangents ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowTangentsChecked() const
{
	return bDrawTangents;
}

void FStaticMeshEditor::ToggleShowBinormals()
{
	bDrawBinormals= !bDrawBinormals;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowBinormalsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowBinormals(bDrawBinormals);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowBinormalsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawBinormals"), bDrawBinormals? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowBinormalsChecked() const
{
	return bDrawBinormals;
}

void FStaticMeshEditor::ToggleShowPivots()
{
	bDrawPivots = !bDrawPivots;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowPivotsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowPivots(bDrawPivots);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowPivotsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawPivots"), bDrawPivots ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowPivotsChecked() const
{
	return bDrawPivots;
}

void FStaticMeshEditor::ToggleShowVertices()
{
	bDrawVertices = !bDrawVertices;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowVerticesFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowVertices(bDrawVertices);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowVerticesFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawVertices"), bDrawVertices ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowVerticesChecked() const
{
	return bDrawVertices;
}

void FStaticMeshEditor::ToggleShowGrids()
{
	bDrawGrids = !IsShowGridsChecked();
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowGridFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowGrids(bDrawGrids);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowGridFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawGrids"), bDrawGrids ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowGridsChecked() const
{
	bool LocalDrawGrids = false;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> CheckShowGridFunc =
		[this, &LocalDrawGrids](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		LocalDrawGrids |= StaticMeshEditorViewportClient.IsSetShowGridChecked();
	};

	ViewportTabContent->PerformActionOnViewports(CheckShowGridFunc);

	return LocalDrawGrids;
}


void FStaticMeshEditor::ToggleShowBounds()
{
	bDrawBounds = !bDrawBounds;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowBoundsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowBounds(bDrawBounds);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowBoundsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawBounds"), bDrawBounds ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowBoundsChecked() const
{
	return bDrawBounds;
}


void FStaticMeshEditor::ToggleShowSimpleCollisions()
{
	bDrawSimpleCollisions = !bDrawSimpleCollisions;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowSimpleCollisionsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowSimpleCollisions(bDrawSimpleCollisions);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowSimpleCollisionsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawSimpleCollisions"), bDrawSimpleCollisions ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowSimpleCollisionsChecked() const
{
	return bDrawSimpleCollisions;
}


void FStaticMeshEditor::ToggleShowComplexCollisions()
{
	bDrawComplexCollisions = !bDrawComplexCollisions;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowComplexCollisionsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowComplexCollisions(bDrawComplexCollisions);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowComplexCollisionsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawComplexCollisions"), bDrawComplexCollisions ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowComplexCollisionsChecked() const
{
	return bDrawComplexCollisions;
}

void FStaticMeshEditor::ToggleShowSockets()
{
	bDrawSockets = !bDrawSockets;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowSocketsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		StaticMeshEditorViewportClient.SetShowPivots(bDrawSockets);
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowSocketsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawSockets"), bDrawSockets ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowSocketsChecked() const
{
	return bDrawSockets;
}

void FStaticMeshEditor::ToggleShowWireframes()
{
	bDrawWireframes = !bDrawWireframes;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowWireframesFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		if (StaticMeshEditorViewport->IsInViewModeWireframeChecked() != bDrawWireframes)
		{
			StaticMeshEditorViewport->SetViewModeWireframe();
		}
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowWireframesFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawWireframes"), bDrawWireframes ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowWireframesChecked() const
{
	return bDrawWireframes;
}

void FStaticMeshEditor::ToggleShowVertexColors()
{
	bDrawVertexColors = !bDrawVertexColors;

	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleShowVertexColorsFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		if (StaticMeshEditorViewport->IsInViewModeVertexColorChecked() != bDrawVertexColors)
		{
 			FStaticMeshEditorViewportClient& EditorViewportClient = StaticMeshEditorViewport->GetViewportClient();

			EditorViewportClient.EngineShowFlags.SetVertexColors(bDrawVertexColors);
			EditorViewportClient.EngineShowFlags.SetLighting(!bDrawVertexColors);
			EditorViewportClient.EngineShowFlags.SetIndirectLightingCache(!bDrawVertexColors);
			EditorViewportClient.EngineShowFlags.SetPostProcessing(!bDrawVertexColors);
			EditorViewportClient.SetFloorAndEnvironmentVisibility(!bDrawVertexColors);
			GetStaticMeshComponent()->bDisplayVertexColors = bDrawVertexColors;
			GetStaticMeshComponent()->MarkRenderStateDirty();
			StaticMeshEditorViewport->Invalidate();
		}
	};

	ViewportTabContent->PerformActionOnViewports(ToggleShowVertexColorsFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawVertexColors"), bDrawVertexColors ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsShowVertexColorsChecked() const
{
	return bDrawVertexColors;
}

void FStaticMeshEditor::ResetCamera()
{
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ResetCameraFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		StaticMeshEditorViewport->GetViewportClient().FocusViewportOnBox(StaticMeshEditorViewport->GetStaticMeshComponent()->Bounds.GetBox());
		StaticMeshEditorViewport->Invalidate();
	};

	ViewportTabContent->PerformActionOnViewports(ResetCameraFunc);

// 	if (FEngineAnalytics::IsAvailable())
// 	{
// 		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("ResetCamera"));
// 	}
}

void FStaticMeshEditor::ToggleDrawAdditionalData()
{
	bDrawAdditionalData = !bDrawAdditionalData;
	TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ToggleDrawAdditionalDataFunc =
		[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
	{
		TSharedRef<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastSharedRef<SStaticMeshEditorViewport>(Entity->AsWidget());
		FStaticMeshEditorViewportClient& StaticMeshEditorViewportClient = StaticMeshEditorViewport->GetViewportClient();
		if (StaticMeshEditorViewportClient.IsDrawAdditionalDataChecked() != bDrawAdditionalData)
		{
			StaticMeshEditorViewportClient.ToggleDrawAdditionalData();
		}
	};

	ViewportTabContent->PerformActionOnViewports(ToggleDrawAdditionalDataFunc);

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawAdditionalData"), bDrawAdditionalData ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditor::IsDrawAdditionalDataChecked() const
{
	return bDrawAdditionalData;
}

void FStaticMeshEditor::BakeMaterials()
{
	if (StaticMesh != nullptr)
	{	
		const IMeshMergeModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities");
		Module.GetUtilities().BakeMaterialsForMesh(StaticMesh);
	}
}

void FStaticMeshEditor::RemoveCurrentUVChannel()
{
	if (!StaticMesh)
	{
		return;
	}

	int32 UVChannelIndex = GetCurrentUVChannel();
	int32 LODIndex = GetCurrentLODIndex();

	FText RemoveUVChannelText = FText::Format(LOCTEXT("ConfirmRemoveUVChannel", "Please confirm removal of UV Channel {0} from LOD {1} of {2}?"), UVChannelIndex, LODIndex, FText::FromString(StaticMesh->GetName()));
	if (FMessageDialog::Open(EAppMsgType::YesNo, RemoveUVChannelText) == EAppReturnType::Yes)
	{
		FMeshBuildSettings& LODBuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;

		if (LODBuildSettings.bGenerateLightmapUVs)
		{
			FText LightmapText;
			if (UVChannelIndex == LODBuildSettings.SrcLightmapIndex)
			{
				LightmapText = FText::Format(LOCTEXT("ConfirmDisableSourceLightmap", "UV Channel {0} is currently used as source for lightmap UVs. Please change the \"Source Lightmap Index\" value or disable \"Generate Lightmap UVs\" in the Build Settings."), UVChannelIndex);
			}
			else if (UVChannelIndex == LODBuildSettings.DstLightmapIndex)
			{
				LightmapText = FText::Format(LOCTEXT("ConfirmDisableDestLightmap", "UV Channel {0} is currently used as destination for lightmap UVs. Please change the \"Destination Lightmap Index\" value or disable \"Generate Lightmap UVs\" in the Build Settings."), UVChannelIndex);
			}

			if (!LightmapText.IsEmpty())
			{
				FMessageDialog::Open(EAppMsgType::Ok, LightmapText);
				return;
			}
		}

		const FScopedTransaction Transaction(LOCTEXT("RemoveUVChannel", "Remove UV Channel"));
		if (StaticMesh->RemoveUVChannel(LODIndex, UVChannelIndex))
		{
			RefreshTool();
		}
	}
}

void FStaticMeshEditor::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingStarted(Toolkit);
}


void FStaticMeshEditor::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
}

#undef LOCTEXT_NAMESPACE
