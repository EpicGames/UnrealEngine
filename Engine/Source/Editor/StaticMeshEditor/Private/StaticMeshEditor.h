// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Toolkits/IToolkitHost.h"
#include "Misc/NotifyHook.h"
#include "EditorUndoClient.h"
#include "IStaticMeshEditor.h"
#include "ISocketManager.h"
#include "TickableEditorObject.h"
#include "SEditorViewport.h"
#include "AdvancedPreviewSceneModule.h"
#include "AssetEditorViewportLayout.h"

// Set USE_ASYNC_DECOMP to zero to go back to the fully synchronous; blocking version of V-HACD
#ifndef USE_ASYNC_DECOMP
#define USE_ASYNC_DECOMP 1
#endif

#if USE_ASYNC_DECOMP
class IDecomposeMeshToHullsAsync;
#endif

class FStaticMeshDetails;
class FEditorViewportClient;
class IDetailsView;
class SConvexDecomposition;
class SDockTab;
class SStaticMeshEditorViewport;
class UStaticMesh;
class UStaticMeshComponent;
class UStaticMeshSocket;
class FViewportTabContent;
struct FPropertyChangedEvent;
struct FTabSpawnerEntry;
class FStaticMeshEditorModeUILayer;

/**
 * StaticMesh Editor class
 */
class FStaticMeshEditor : public IStaticMeshEditor, public FGCObject, public FEditorUndoClient, public FNotifyHook, public FTickableEditorObject
{
public:
	FStaticMeshEditor()
		: StaticMesh(nullptr)
		, MinPrimSize(0.5f)
		, OverlapNudge(10.0f)
		, CurrentViewedUVChannel(0)
		, SecondaryToolbarEntry(nullptr)
		, bDrawNormals(false)
		, bDrawTangents(false)
		, bDrawBinormals(false)
		, bDrawPivots(false)
		, bDrawVertices(false)
		, bDrawGrids(false)
		, bDrawBounds(false)
		, bDrawSimpleCollisions(false)
		, bDrawComplexCollisions(false)
		, bDrawSockets(false)
		, bDrawWireframes(false)
		, bDrawVertexColors(false)
		, bDrawAdditionalData(true)
	{}

	~FStaticMeshEditor();

private:
	/** Initializes the editor to use a static mesh. Should be the first thing called. */
	void InitEditorForStaticMesh(UStaticMesh* ObjectToEdit);

	virtual void PostInitAssetEditor() override;
	virtual void OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source) override;

public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	// IToolkitHost Interface
	void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;

	/**
	 * Edits the specified static mesh object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The static mesh to edit
	 */
	void InitStaticMeshEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStaticMesh* ObjectToEdit);

	/** Creates details for a static mesh */
	TSharedRef<class IDetailCustomization> MakeStaticMeshDetails();

	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FStaticMeshEditor");
	}
	//~ End FGCObject Interface

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Content/Types/StaticMeshes/Editor"));
	}

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** IStaticMeshEditor interface */
	virtual UStaticMesh* GetStaticMesh() override { return StaticMesh; }
	virtual UStaticMeshComponent* GetStaticMeshComponent() const override;

	virtual UStaticMeshSocket* GetSelectedSocket() const override;
	virtual TArray<UStaticMeshSocket*> GetSelectedSockets() const override;
	virtual bool HasSelectedSockets() const override;
	virtual void SetSelectedSocket(UStaticMeshSocket* InSelectedSocket) override;
	virtual void AddSelectedSocket(UStaticMeshSocket* InSelectedSocket) override;
	virtual void RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket) override;
	virtual void DuplicateSelectedSocket() override;
	virtual void DuplicateSelectedSockets() override;
	virtual void RequestRenameSelectedSocket() override;

	virtual bool IsPrimValid(const FPrimData& InPrimData) const override;
	virtual bool HasSelectedPrims() const override;
	virtual const TArray<FPrimData>& GetSelectedPrims() const override;
	virtual void AddSelectedPrim(const FPrimData& InPrimData, bool bClearSelection) override;
	virtual void RemoveSelectedPrim(const FPrimData& InPrimData) override;
	virtual void RemoveInvalidPrims() override;
	virtual bool IsSelectedPrim(const FPrimData& InPrimData) const override;
	virtual void ClearSelectedPrims() override;
	virtual void DuplicateSelectedPrims(const FVector* InOffset) override;
	virtual int32 CopySelectedPrims() const override;
	virtual int32 PasteCopiedPrims() override;
	virtual void TranslateSelectedPrims(const FVector& InDrag) override;
	virtual void RotateSelectedPrims(const FRotator& InRot) override;
	virtual void ScaleSelectedPrims(const FVector& InScale) override;
	virtual bool CalcSelectedPrimsAABB(FBox &OutBox) const override;
	virtual bool GetLastSelectedPrimTransform(FTransform& OutTransform) const override;
	FTransform GetPrimTransform(const FPrimData& InPrimData) const override;
	void SetPrimTransform(const FPrimData& InPrimData, const FTransform& InPrimTransform) const override;
	bool OverlapsExistingPrim(const FPrimData& InPrimData) const;

	virtual int32 GetNumTriangles(int32 LODLevel = 0) const override;
	virtual int32 GetNumVertices(int32 LODLevel = 0) const override;
	virtual int32 GetNumUVChannels(int32 LODLevel = 0) const override;

	virtual int32 GetCurrentUVChannel() override;
	virtual int32 GetCurrentLODLevel() override;
	virtual int32 GetCurrentLODIndex() override;

	virtual int32 GetCustomData(const int32 Key) const override;
	virtual void SetCustomData(const int32 Key, const int32 CustomData) override;

	virtual void RefreshTool() override;
	virtual void RefreshViewport() override;
	virtual bool IsShowSimpleCollisionChecked() const override;
	virtual void ToggleShowSimpleCollision() override;
	virtual void DoDecomp(uint32 InHullCount, int32 InMaxHullVerts, uint32 InHullPrecision) override;

	virtual TSet< int32 >& GetSelectedEdges() override;

	virtual FEditorViewportClient& GetViewportClient() override;

	virtual void SetSecondaryToolbarDisplayName(FText DisplayName) override;
	// End of IStaticMeshEditor

	/** Extends the toolbar menu to include static mesh editor options */
	void ExtendMenu();

	/** Registers a delegate to be called after an Undo operation */
	virtual void RegisterOnPostUndo(const FOnPostUndo& Delegate) override;

	/** Unregisters a delegate to be called after an Undo operation */
	virtual void UnregisterOnPostUndo(SWidget* Widget) override;

	/** From FNotifyHook */
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	/** Get the active view mode */
	virtual EViewModeIndex GetViewMode() const override;

	virtual void RegisterOnSelectedLODChanged(const FOnSelectedLODChanged &Delegate, bool UnregisterOnRefresh) override
	{
		if (!UnregisterOnRefresh)
		{
			OnSelectedLODChanged.Add(Delegate);
		}
		else
		{
			OnSelectedLODChangedResetOnRefresh.Add(Delegate);
		}
	}

	virtual void UnRegisterOnSelectedLODChanged(FDelegateUserObject Thing) override
	{
		OnSelectedLODChanged.RemoveAll(Thing);
		OnSelectedLODChangedResetOnRefresh.RemoveAll(Thing);
	}

	virtual void BroadcastOnSelectedLODChanged() override
	{
		if (OnSelectedLODChanged.IsBound())
		{
			OnSelectedLODChanged.Broadcast();
		}
	}

	/** For asynchronous convex decomposition support, this class is tickable in the editor to be able to confirm
	that the process is completed */
	virtual bool IsTickableInEditor() const final
	{
		return true;
	}

	/** This is a tickable class */
	virtual ETickableTickType GetTickableTickType() const final
	{
		return ETickableTickType::Always;
	}

	/** Performs the main 'tick' operation on this class.  The 'tick' step checks to see if there is currently
	an active asynchronous convex decomopsition task running and, if so, checks to see if it is completed and,
	if so, gathers the results and releases the interface */
	virtual void Tick(float DeltaTime) final;

	/** Returns the stat ID for this tickable class */
	virtual TStatId GetStatId() const final;

	/** Add a widget to the StaticMeshViewport's ViewportOverlay */
	void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE) override;

	/** Remove a widget from the StaticMeshViewport's ViewportOverlay */
	void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;

	void CreateEditorModeManager() override;

	/**	The tab ids for all the tabs used */
	static const FName ViewportTabId;
	static const FName PropertiesTabId;
	static const FName SocketManagerTabId;
	static const FName CollisionTabId;
	static const FName PreviewSceneSettingsTabId;
	static const FName SecondaryToolbarTabId;

private:
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SocketManager(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Collision(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SecondaryToolbar(const FSpawnTabArgs& Args);


	/** Callback for toggling the normals show flag. */
	void ToggleShowNormals();

	/** Callback for checking the normals show flag. */
	bool IsShowNormalsChecked() const;

	/** Callback for toggling the tangents show flag. */
	void ToggleShowTangents();

	/** Callback for checking the tangents show flag. */
	bool IsShowTangentsChecked() const;

	/** Callback for toggling the binormals show flag. */
	void ToggleShowBinormals();

	/** Callback for checking the binormals show flag. */
	bool IsShowBinormalsChecked() const;

	/** Callback for toggling the pivots show flag. */
	void ToggleShowPivots();

	/** Callback for checking the pivots show flag. */
	bool IsShowPivotsChecked() const;

	/** Callback for toggling the vertices show flag. */
	void ToggleShowVertices();

	/** Callback for checking the vertices show flag. */
	bool IsShowVerticesChecked() const;

	/** Callback for toggling the grid show flag. */
	void ToggleShowGrids();

	/** Callback for checking the grid show flag. */
	bool IsShowGridsChecked() const;

	/** Callback for toggling the bounds show flag. */
	void ToggleShowBounds();

	/** Callback for checking the bounds show flag. */
	bool IsShowBoundsChecked() const;

	/** Callback for toggling the simple collisions show flag. */
	void ToggleShowSimpleCollisions();

	/** Callback for checking the simple collisions show flag. */
	bool IsShowSimpleCollisionsChecked() const;

	/** Callback for toggling the complex collisions show flag. */
	void ToggleShowComplexCollisions();

	/** Callback for checking the complex collisions show flag. */
	bool IsShowComplexCollisionsChecked() const;

	/** Callback for toggling the sockets show flag. */
	void ToggleShowSockets();

	/** Callback for checking the sockets show flag. */
	bool IsShowSocketsChecked() const;

	/** Callback for toggling the wireframes show flag. */
	void ToggleShowWireframes();

	/** Callback for checking the wireframes show flag. */
	bool IsShowWireframesChecked() const;

	/** Callback for toggling the vertex colors show flag. */
	void ToggleShowVertexColors();

	/** Callback for checking the vertex colors show flag. */
	bool IsShowVertexColorsChecked() const;

	/** Reset the viewport camera to look at the mesh. */
	void ResetCamera();

	/** Callback for toggling the draw additional data flag. */
	void ToggleDrawAdditionalData();

	/** Callback for checking the draw additional data flag. */
	bool IsDrawAdditionalDataChecked() const;

	/** Bake out Materials for give LOD(s). **/
	void BakeMaterials();

private:


	/** Gets First Viewport from layout. */
	TSharedPtr<class SStaticMeshEditorViewport> GetStaticMeshViewport() const;

	FAdvancedPreviewSceneModule::FOnPreviewSceneChanged OnPreviewSceneChangedDelegate;

	/** Called when the Viewport Layout has changed. */
	void OnEditorLayoutChanged();

	/** Binds commands associated with the Static Mesh Editor. */
	void BindCommands();

	/** Builds the Static Mesh Editor toolbar. */
	void ExtendToolBar();

	/** Builds the sub-tools that are a part of the static mesh editor. */
	void BuildSubTools();

	/**
	* Updates NumTriangles, NumVertices and NumUVChannels for the given LOD
	*/
	void UpdateLODStats(int32 CurrentLOD);

	/** A general callback for the combo boxes in the Static Mesh Editor to force a viewport refresh when a selection changes. */
	void ComboBoxSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	struct FReimportParameters
	{
		FReimportParameters(bool bInWithNewFile, bool bInWithDialog)
			: bWithNewFile(bInWithNewFile)
			, bWithDialog(bInWithDialog)
		{}

		bool bWithNewFile = false;
		bool bWithDialog = false;
	};

	/* Callback to reimport the base mesh*/
	void HandleReimportMesh(const FReimportParameters ReimportParameters);
	
	/* Callback to reimport the base mesh and also all custom LODs*/
	void HandleReimportAllMesh(const FReimportParameters ReimportParameters);

	/**
	 *	Sets the editor's current mesh and refreshes various settings to correspond with the new data.
	 *
	 *	@param	InStaticMesh		The static mesh to use for the editor.
	 */
	void SetEditorMesh(UStaticMesh* InStaticMesh, bool bResetCamera = true);

	/** Helper function for generating K-DOP collision geometry. */
	void GenerateKDop(const FVector* Directions, uint32 NumDirections);

	/** Callback for creating box collision. */
	void OnCollisionBox();

	/** Callback for creating sphere collision. */
	void OnCollisionSphere();

	/** Callback for creating sphyl collision. */
	void OnCollisionSphyl();

	/**
	* Quick and dirty way of creating box vertices from a box collision representation
	* Grossly inefficient, but not time critical
	* @param BoxElem - Box collision to get the vertices for
	* @param Verts - output listing of vertex data from the box collision
	* @param Scale - scale to create this box at
	*/
	void CreateBoxVertsFromBoxCollision(const struct FKBoxElem& BoxElem, TArray<FVector>& Verts, float Scale);

	/** Converts the collision data for the static mesh */
	void OnConvertBoxToConvexCollision(void);

	/** Copy the collision data from the selected static mesh in Content Browser*/
	void OnCopyCollisionFromSelectedStaticMesh(void);

	/** Whether there is a valid static mesh to copy collision from selected in the content browser */
	bool CanCopyCollisionFromSelectedStaticMesh() const;

	/** Get the first static mesh selected in the content browser */
	UStaticMesh* GetFirstSelectedStaticMeshInContentBrowser() const;

	/** Clears the collision data for the static mesh */
	void OnRemoveCollision(void);

	/** Whether there is collision to remove from the static mesh */
	bool CanRemoveCollision();

	/** Change the mesh the editor is viewing. */
	void OnChangeMesh();

	/** Whether there is a static mesh selected in the content browser to change to*/
	bool CanChangeMesh() const;

	/** Replace the generated LODs in the original source mesh with the reduced versions.*/
	void OnSaveGeneratedLODs();

	/** Rebuilds the UV Channel combo list and attempts to set it to the same channel. */
	void GenerateUVChannelComboList(UToolMenu* InMenu);

	/** Delete whats currently selected */
	void DeleteSelected();

	/** Whether we currently have any selected that can be deleted */
	bool CanDeleteSelected() const;

	/** Init SocketManager (does nothing if it already exists) */
	void InitSocketManager();

	/** Delete the currently selected sockets */
	void DeleteSelectedSockets();

	/** Delete the currently selected prims */
	void DeleteSelectedPrims();

	/** Duplicate whats currently selected */
	void DuplicateSelected();

	/** Whether we currently have any selected that can be duplicated */
	bool CanDuplicateSelected() const;

	/** Copy what's currently selected to the clipboard */
	void CopySelected();

	/** Whether we currently have anything selected that can be copied to the clipboard */
	bool CanCopySelected() const;

	/** Paste what's current on the clipboard */
	void PasteCopied();

	/** Whether we can paste what's currently on the clipboard */
	bool CanPasteCopied() const;

	/** Whether we currently have any selected that can be renamed */
	bool CanRenameSelected() const;

	/** Handler for when FindInExplorer is selected */
	void ExecuteFindInExplorer();

	/** Returns true to allow execution of source file commands */
	bool CanExecuteSourceCommands() const;

	/** Callback when an object is reimported, handles steps needed to keep the editor up-to-date. */
	void OnObjectReimported(UObject* InObject);

	/** Opens the convex decomposition tab. */
	void OnConvexDecomposition();

	//~ Begin FAssetEditorToolkit Interface.
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual void SetupReadOnlyMenuProfiles(FReadOnlyAssetEditorCustomization& OutReadOnlyCustomization) override;

	//~ End FAssetEditorToolkit Interface.

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Undo Action**/
	void UndoAction();

	/** Redo Action **/
	void RedoAction();

	/** Callback when the mesh change **/
	void OnMeshChanged();

	/** Called when socket selection changes */
	void OnSocketSelectionChanged();

	/** Callback when an object has been reimported, and whether it worked */
	void OnPostReimport(UObject* InObject, bool bSuccess);

	void SetCurrentViewedUVChannel(int32 InNewUVChannel);

	ECheckBoxState GetUVChannelCheckState(int32 TestUVChannel) const;

	/** Callbacks for UV edition */
	bool CanRemoveUVChannel();
	void RemoveCurrentUVChannel();

	/** Adds or removes extenders to the secondary toolbar */
	void AddSecondaryToolbarExtender(TSharedPtr<FExtender> Extender);
	void RemoveSecondaryToolbarExtender(TSharedPtr<FExtender> Extender);

	void GenerateSecondaryToolbar();

private:
	// Tracking the active viewports in this editor.
	TSharedPtr<class FEditorViewportTabContent> ViewportTabContent;

	/** Property View */
	TSharedPtr<class IDetailsView> StaticMeshDetailsView;

	/** Socket Manager widget. */
	TSharedPtr< class ISocketManager> SocketManager;

	/** Convex Decomposition widget */
	TSharedPtr< class SConvexDecomposition> ConvexDecomposition;

	/** Static mesh editor detail customization */
	TWeakPtr<class FStaticMeshDetails> StaticMeshDetails;

	/** The currently viewed Static Mesh. */
	TObjectPtr<UStaticMesh> StaticMesh;

	/** The number of triangles associated with the static mesh LOD. */
	TArray<int32> NumTriangles;

	/** The number of vertices associated with the static mesh LOD. */
	TArray<int32> NumVertices;

	/** The number of used UV channels. */
	TArray<int32> NumUVChannels;

	/** Delegates called after an undo operation for child widgets to refresh */
	FOnPostUndoMulticaster OnPostUndo;

	/** Information on the selected collision primitives */
	TArray<FPrimData> SelectedPrims;

	/** Scene preview settings widget */
	TSharedPtr<SWidget> AdvancedPreviewSettingsWidget;

	/** Misc consts */
	const float MinPrimSize;
	const FVector OverlapNudge;

	/** The current UV Channel we are viewing */
	int32 CurrentViewedUVChannel;

	/** Selected LOD changed delegates */
	FOnSelectedLODChangedMulticaster OnSelectedLODChanged;
	FOnSelectedLODChangedMulticaster OnSelectedLODChangedResetOnRefresh;

#if USE_ASYNC_DECOMP
	/** Instance of the active asynchronous convex decomposition interface. */
	IDecomposeMeshToHullsAsync        *DecomposeMeshToHullsAsync{ nullptr };
#endif

	/** Allow custom data for this editor */
	TMap<int32, int32> CustomEditorData;

	/** A local reference to the private Extender set in FAssetEditorToolkit so that we can extend our Toolbar sub-menus */
	TSharedPtr<FExtender> EditorToolbarExtender;

	/** Static Mesh Editor Secondary Toolbar */
	TSharedPtr<SWidget> SecondaryToolbar;

	/** The widget that will house the secondary toolbar widget */
	TSharedPtr<SBorder> SecondaryToolbarWidgetContent;

	/** The extenders to populate the secondary toolbar with */
	TArray<TSharedPtr<FExtender>> SecondaryToolbarExtenders;

	/** The tab that the preview scene settings widget goes in */
	TWeakPtr<SDockTab> PreviewSceneDockTab;

	/** Spawner entry for the secondary toolbar tab */
	FTabSpawnerEntry* SecondaryToolbarEntry;

	/** The text display name to override the default display name of the secondary toolbar*/
	FText SecondaryToolbarDisplayName;
	
	/** Storage for our viewport creation function that will be passed to the viewport layout system*/
	AssetEditorViewportFactoryFunction MakeViewportFunc;

	/** Toolbar toggles */
	bool bDrawNormals;
	bool bDrawTangents;
	bool bDrawBinormals;
	bool bDrawPivots;
	bool bDrawVertices;
	bool bDrawGrids;
	bool bDrawBounds;
	bool bDrawSimpleCollisions;
	bool bDrawComplexCollisions;
	bool bDrawSockets;
	bool bDrawWireframes;
	bool bDrawVertexColors;
	bool bDrawAdditionalData;

	TSharedPtr<FStaticMeshEditorModeUILayer> ModeUILayer;
};
