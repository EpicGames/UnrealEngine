// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorActions.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditorCommands"

void FStaticMeshEditorCommands::RegisterCommands()
{
	UI_COMMAND(SetShowNaniteFallback, "Nanite Fallback", "Toggles the display of the Nanite fallback mesh. The fallback mesh is used when Nanite is not available such as platforms that do not support it.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::N));
	UI_COMMAND(SetShowDistanceField, "Visualize Distance Field", "Toggles the display of the mesh distance field.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowRayTracingFallback, "Ray Tracing Fallback", "Toggles the display of the ray tracing fallback mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowWireframe, "Wireframe", "Toggles the viewmode of the Preview Pane between a lit view and a wireframe view.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowVertexColor, "Vert Colors", "Toggles vertex colors.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowPhysicalMaterialMasks, "Physical Material Masks", "Toggles physical material masks.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetDrawUVs, "UV", "Toggles display of the static mesh's UVs for the specified channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::G, EModifierKey::Alt | EModifierKey::Shift));
	UI_COMMAND(SetShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowSimpleCollision, "Simple Collision", "Toggles display of the simplified collision mesh of the static mesh, if present.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowComplexCollision, "Complex Collision", "Toggles display of the complex collision for this static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ResetCamera, "Reset Camera", "Resets the camera to focus on the static mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetShowSockets, "Sockets", "Displays the static mesh sockets.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::S));
	UI_COMMAND(SetDrawAdditionalData, "Additional Data", "Draw additional user data associated with asset.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ReimportMesh, "Reimport Mesh", "Reimport the base mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportMeshWithNewFile, "Reimport Mesh With New File", "Reimport the base mesh using a new source file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMesh, "Reimport Mesh + LODs", "Reimport the base mesh and all the custom LODs.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMeshWithNewFile, "Reimport Mesh + LODs With New File", "Reimport the base mesh using a new source file and all the custom LODs (No new source file for the custom LODs).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportMeshWithDialog, "Reimport Mesh With Dialog", "Reimport the base mesh and show the re-import dialog.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportMeshWithNewFileWithDialog, "Reimport Mesh With New File", "Reimport the base mesh using a new source file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMeshWithDialog, "Reimport Mesh + LODs", "Reimport the base mesh and all the custom LODs.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportAllMeshWithNewFileWithDialog, "Reimport Mesh + LODs With New File", "Reimport the base mesh using a new source file and all the custom LODs (No new source file for the custom LODs).", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowTangents, "Tangents", "Toggles display of vertex tangents in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowBinormals, "Binormals", "Toggles display of vertex binormals (orthogonal vector to normal and tangent) in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowVertices, "Vertices", "Toggles display of vertices in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowPivots, "Show Pivot", "Display the pivot location of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowGrids, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowSimpleCollisions, "Simple Collision", "Toggles display of the simplified collision mesh of the static mesh, if present.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowComplexCollisions, "Complex Collision", "Toggles display of the complex collision for this static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowSockets, "Sockets", "Displays the static mesh sockets.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowWireframes, "Wireframe", "Toggles the viewmode of the Preview Pane between a lit view and a wireframe view.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowVertexColors, "Vert Colors", "Toggles vertex colors.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::N));
	UI_COMMAND(SetShowTangents, "Tangents", "Toggles display of vertex tangents in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::T));
	UI_COMMAND(SetShowBinormals, "Binormals", "Toggles display of vertex binormals (orthogonal vector to normal and tangent) in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::B));
	UI_COMMAND(SetShowVertices, "Vertices", "Toggles display of vertices in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::V));
	UI_COMMAND(SetShowPivot, "Show Pivot", "Display the pivot location of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::P));

	UI_COMMAND(CreateDOP10X, "Add 10DOP-X Simplified Collision", "Generates a new axis-aligned box collision mesh with the 4 X-axis aligned edges beveled (10 total sides) encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateDOP10Y, "Add 10DOP-Y Simplified Collision", "Generates a new axis-aligned box collision mesh with the 4 Y-axis aligned edges beveled (10 total sides) encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateDOP10Z, "Add 10DOP-Z Simplified Collision", "Generates a new axis-aligned box collision mesh with the 4 Z-axis aligned edges beveled (10 total sides) encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateDOP18, "Add 18DOP Simplified Collision", "Generates a new axis-aligned box collision mesh with all edges beveled (18 total sides) encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateDOP26, "Add 26DOP Simplified Collision", "Generates a new axis-aligned box collision mesh with all edges and corners beveled (26 total sides) encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CreateBoxCollision, "Add Box Simplified Collision", "Generates a new box collision mesh encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateSphereCollision, "Add Sphere Simplified Collision", "Generates a new sphere collision mesh encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateSphylCollision, "Add Capsule Simplified Collision", "Generates a new capsule collision mesh encompassing the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateAutoConvexCollision, "Auto Convex Collision", "Opens the Auto Convex Collision Tool for generating a new convex collision mesh, or meshes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveCollision, "Remove Collision", "Removes any simplified collision assigned to the static mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertBoxesToConvex, "Convert Boxes to Convex", "Converts any simple box collision meshes to convex collision meshes.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CopyCollisionFromSelectedMesh, "Copy Collision from Selected Static Mesh", "Copy collision from the static mesh selected in Content Browser.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(FindSource, "Find Source", "Opens explorer at the location of this asset.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ChangeMesh, "Change Mesh", "Changes the static mesh asset loaded in the Static Mesh Editor to the asset currently selected in the Content Browser.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SaveGeneratedLODs, "Save Generated LODs", "Saves the generated LODs in the mesh package.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BakeMaterials, "Bake Out Materials", "Bake out Materials for given LOD(s).", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SetAspectRatio_Free, "Unconstrained", "Sets the aspect ratio to be unconstrained.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_Platform, "Platform", "Sets the aspect ratio to be controlled by the current preview platform, if any.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_6_13, "6:13", "Sets the aspect ratio to be 6:13 (Mobile Portrait Orientation).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_13_6, "13:6", "Sets the aspect ratio to be 6:13 (Mobile Landscape Orientation).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_1_1, "1:1", "Sets the aspect ratio to be 1:1 (Square).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_4_3, "4:3", "Sets the aspect ratio to be 4:3 (Fullscreen).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_16_10, "16:10", "Sets the aspect ratio to be 16:10 (Laptop).", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetAspectRatio_16_9, "16:9", "Sets the aspect ratio to be 16:9 (HD)", EUserInterfaceActionType::RadioButton, FInputChord());	
	


}

#undef LOCTEXT_NAMESPACE
