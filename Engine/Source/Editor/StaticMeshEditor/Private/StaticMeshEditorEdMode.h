// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"

/**
 * EdMode for the StaticMeshEditor, that enables various ITF functionality.
 */
class FStaticMeshEditorEdMode : public FEdMode
{
public:
	static inline const FEditorModeID Id = TEXT("StaticMeshEditorEdMode");

	virtual bool ShouldDrawWidget() const override { return true; }
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	// Explicitly handle duplication. Reconsider when sockets have TypedElement handlers.
	virtual EEditAction::Type GetActionEditDuplicate() override { return EEditAction::Process; }
};
