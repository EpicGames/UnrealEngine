// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModeUILayer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Toolkits/IToolkit.h"
#include "StaticMeshEditor.h"
#include "StaticMeshEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshEditorModeUILayer)

void UStaticMeshEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	IStaticMeshEditorModule& StaticMeshEditorModule = FModuleManager::GetModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
	StaticMeshEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UStaticMeshEditorUISubsystem::RegisterLayoutExtensions);
}

void UStaticMeshEditorUISubsystem::Deinitialize()
{
	IStaticMeshEditorModule& StaticMeshEditorModule = FModuleManager::GetModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
	StaticMeshEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void UStaticMeshEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
	Extender.ExtendLayout(FStaticMeshEditor::SocketManagerTabId, ELayoutExtensionPosition::After, NewTab);
}

FStaticMeshEditorModeUILayer::FStaticMeshEditorModeUILayer(const IToolkitHost* InToolkitHost) :
	FAssetEditorModeUILayer(InToolkitHost)
{}

void FStaticMeshEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();
		OnToolkitHostReadyForUI.ExecuteIfBound();
	}
}

void FStaticMeshEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}
}

TSharedPtr<FWorkspaceItem> FStaticMeshEditorModeUILayer::GetModeMenuCategory() const
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	return MenuStructure.GetLevelEditorModesCategory();
}

