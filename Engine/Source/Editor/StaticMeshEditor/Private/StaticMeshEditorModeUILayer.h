// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"
#include "StaticMeshEditorModeUILayer.generated.h"

UCLASS()
class UStaticMeshEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;
};


class FStaticMeshEditorModeUILayer : public FAssetEditorModeUILayer
{
public:

	FStaticMeshEditorModeUILayer(const IToolkitHost* InToolkitHost);

	void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	
	TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;
};

