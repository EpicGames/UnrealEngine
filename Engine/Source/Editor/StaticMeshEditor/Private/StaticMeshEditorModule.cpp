// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModule.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "IStaticMeshEditor.h"
#include "StaticMeshEditor.h"
#include "StaticMeshEditorEdMode.h"

const FName StaticMeshEditorAppIdentifier = FName(TEXT("StaticMeshEditorApp"));

/**
 * StaticMesh editor module
 */
class FStaticMeshEditorModule : public IStaticMeshEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FStaticMeshEditorModule()
	{
	}

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		// Make sure the advanced preview scene module is loaded
		FModuleManager::Get().LoadModuleChecked("AdvancedPreviewScene");

		FEditorModeRegistry::Get().RegisterMode<FStaticMeshEditorEdMode>(FStaticMeshEditorEdMode::Id);

		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
		SecondaryToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	}

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override
	{
		if (UObjectInitialized())
		{
			FEditorModeRegistry::Get().UnregisterMode(FStaticMeshEditorEdMode::Id);
		}

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();
		SecondaryToolBarExtensibilityManager.Reset();
	}

	/**
	 * Creates a new StaticMesh editor for a StaticMesh
	 */
	virtual TSharedRef<IStaticMeshEditor> CreateStaticMeshEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UStaticMesh* StaticMesh ) override
	{
		TSharedRef<FStaticMeshEditor> NewStaticMeshEditor(new FStaticMeshEditor());
		NewStaticMeshEditor->InitStaticMeshEditor(Mode, InitToolkitHost, StaticMesh);
		return NewStaticMeshEditor;
	}

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetSecondaryToolBarExtensibilityManager() override { return SecondaryToolBarExtensibilityManager; }

	virtual TArray<FStaticMeshEditorToolbarExtender>& GetAllStaticMeshEditorToolbarExtenders() override { return StaticMeshEditorToolbarExtenders; }

	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() override { return RegisterLayoutExtensions; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> SecondaryToolBarExtensibilityManager;
	TArray<FStaticMeshEditorToolbarExtender> StaticMeshEditorToolbarExtenders;
	FOnRegisterLayoutExtensions	RegisterLayoutExtensions;
};

IMPLEMENT_MODULE( FStaticMeshEditorModule, StaticMeshEditor );
