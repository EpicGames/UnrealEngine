// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorStyle.h"

#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"

FName FStaticMeshEditorStyle::StyleName("StaticMeshEditorStyle");

FStaticMeshEditorStyle::FStaticMeshEditorStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D IconSize(16.0f, 16.0f);
	const FVector2D ToolbarIconSize(20.0f, 20.0f);

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate/Starship/StaticMeshEditor"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		FSliderStyle SliderStyle = FSliderStyle()
			.SetNormalBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Input, 1.0f))
			.SetHoveredBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Input, 1.0f))
			.SetDisabledBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Recessed, 1.0f))
			.SetNormalThumbImage(*FCoreStyle::Get().GetBrush("TreeArrow_Expanded"))
			.SetHoveredThumbImage(*FCoreStyle::Get().GetBrush("TreeArrow_Expanded"))
			.SetDisabledThumbImage(*FCoreStyle::Get().GetBrush("TreeArrow_Expanded"))
			.SetBarThickness(0.0f);
		Set("StaticMeshEditor.LODMeter.SliderStyle", SliderStyle);
	}

	Set("StaticMeshEditor.LODMeter.DisabledBrush", new IMAGE_BRUSH("DisabledOverlay_x16", IconSize, FLinearColor::Gray, ESlateBrushTileType::Both));
	Set("StaticMeshEditor.LODMeter.NoBrush",new FSlateNoResource());

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FStaticMeshEditorStyle::~FStaticMeshEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FStaticMeshEditorStyle& FStaticMeshEditorStyle::Get()
{
	static FStaticMeshEditorStyle Inst;
	return Inst;
}