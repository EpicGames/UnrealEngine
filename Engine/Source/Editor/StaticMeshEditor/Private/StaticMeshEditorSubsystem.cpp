// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorSubsystem.h"

#include "ActorEditorUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/MeshComponent.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorFramework/AssetImportData.h"
#include "EngineUtils.h"
#include "Engine/Brush.h"
#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "FbxMeshUtils.h"
#include "FileHelpers.h"
#include "GameFramework/Actor.h"
#include "IContentBrowserSingleton.h"
#include "IMeshMergeUtilities.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "LevelEditorViewport.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshOperations.h"
#include "MeshMergeModule.h"
#include "PhysicsEngine/BodySetup.h"
#include "ScopedTransaction.h"
#include "Async/ParallelFor.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Async/Async.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FeedbackContext.h"
#include "Templates/Greater.h"

#include "StaticMeshResources.h"
#include "UnrealEdGlobals.h"
#include "GeomFitUtils.h"
#include "ConvexDecompTool.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "Layers/LayersSubsystem.h"
#include "EditorScriptingHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshEditorSubsystem)

#define LOCTEXT_NAMESPACE "StaticMeshEditorSubsystem"

DEFINE_LOG_CATEGORY(LogStaticMeshEditorSubsystem);

namespace InternalEditorMeshLibrary
{
	/** Note: This method is a replicate of FStaticMeshEditor::DoDecomp */
	bool GenerateConvexCollision(UStaticMesh* StaticMesh, uint32 HullCount, int32 MaxHullVerts, uint32 HullPrecision)
	{
		// Check we have a valid StaticMesh
		if (!StaticMesh || !StaticMesh->IsMeshDescriptionValid(0))
		{
			return false;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(GenerateConvexCollision)

			// If RenderData has not been computed yet, do it
			if (!StaticMesh->GetRenderData())
			{
				StaticMesh->CacheDerivedData();
			}

		const FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[0];

		// Make vertex buffer
		int32 NumVerts = LODModel.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
		TArray<FVector3f> Verts;
		Verts.Reserve(NumVerts);
		for (int32 i = 0; i < NumVerts; i++)
		{
			Verts.Add(LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(i));
		}

		// Grab all indices
		TArray<uint32> AllIndices;
		LODModel.IndexBuffer.GetCopy(AllIndices);

		// Only copy indices that have collision enabled
		TArray<uint32> CollidingIndices;
		for (const FStaticMeshSection& Section : LODModel.Sections)
		{
			if (Section.bEnableCollision)
			{
				for (uint32 IndexIdx = Section.FirstIndex; IndexIdx < Section.FirstIndex + (Section.NumTriangles * 3); IndexIdx++)
				{
					CollidingIndices.Add(AllIndices[IndexIdx]);
				}
			}
		}

		// Do not perform any action if we have invalid input
		if (Verts.Num() < 3 || CollidingIndices.Num() < 3)
		{
			return false;
		}

		// Get the BodySetup we are going to put the collision into
		UBodySetup* BodySetup = StaticMesh->GetBodySetup();
		if (BodySetup)
		{
			BodySetup->RemoveSimpleCollision();
		}
		else
		{
			// Otherwise, create one here.
			StaticMesh->CreateBodySetup();
			BodySetup = StaticMesh->GetBodySetup();
		}

		// Run actual util to do the work (if we have some valid input)
		DecomposeMeshToHulls(BodySetup, Verts, CollidingIndices, HullCount, MaxHullVerts, HullPrecision);

		StaticMesh->SetCustomizedCollision(true);	//mark the static mesh for collision customization

		return true;
	}

	bool IsUVChannelValid(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex)
	{
		if (StaticMesh == nullptr)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "The StaticMesh is null.");
			return false;
		}

		if (LODIndex >= StaticMesh->GetNumLODs() || LODIndex < 0)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "The StaticMesh doesn't have LOD %d.", LODIndex);
			return false;
		}

		if (!StaticMesh->IsMeshDescriptionValid(LODIndex))
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "No mesh description for LOD %d.", LODIndex);
			return false;
		}

		int32 NumUVChannels = StaticMesh->GetNumUVChannels(LODIndex);
		if (UVChannelIndex < 0 || UVChannelIndex >= NumUVChannels)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "The given UV channel index %d is out of bounds.", UVChannelIndex);
			return false;
		}

		return true;
	}

	template<typename ArrayType>
	int32 ReplaceMeshes(const ArrayType& Array, UStaticMesh* MeshToBeReplaced, UStaticMesh* NewMesh)
	{
		//Would use FObjectEditorUtils::SetPropertyValue, but meshes are a special case. They need a lock and we need to use the SetMesh function
		FProperty* StaticMeshProperty = FindFieldChecked<FProperty>(UStaticMeshComponent::StaticClass(), "StaticMesh");
		TArray<UObject*, TInlineAllocator<16>> ObjectsThatChanged;
		int32 NumberOfChanges = 0;

		for (UStaticMeshComponent* Component : Array)
		{
			const bool bIsClassDefaultObject = Component->HasAnyFlags(RF_ClassDefaultObject);
			if (!bIsClassDefaultObject)
			{
				if (Component->GetStaticMesh() == MeshToBeReplaced)
				{
					FEditPropertyChain PropertyChain;
					PropertyChain.AddHead(StaticMeshProperty);
					static_cast<UObject*>(Component)->PreEditChange(PropertyChain);

					// Set the mesh
					Component->SetStaticMesh(NewMesh);
					++NumberOfChanges;

					ObjectsThatChanged.Add(Component);
				}
			}
		}

		// Route post edit change after all components have had their values changed.  This is to avoid
		// construction scripts from re-running in the middle of setting values and wiping out components we need to modify
		for (UObject* ObjectData : ObjectsThatChanged)
		{
			FPropertyChangedEvent PropertyEvent(StaticMeshProperty);
			ObjectData->PostEditChangeProperty(PropertyEvent);
		}

		return NumberOfChanges;
	}

	template<typename ArrayType>
	int32 ReplaceMaterials(ArrayType& Array, UMaterialInterface* MaterialToBeReplaced, UMaterialInterface* NewMaterial)
	{
		//Would use FObjectEditorUtils::SetPropertyValue, but Material are a special case. They need a lock and we need to use the SetMaterial function
		FProperty* MaterialProperty = FindFieldChecked<FProperty>(UMeshComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials));
		TArray<UObject*, TInlineAllocator<16>> ObjectsThatChanged;
		int32 NumberOfChanges = 0;

		for (UMeshComponent* Component : Array)
		{
			const bool bIsClassDefaultObject = Component->HasAnyFlags(RF_ClassDefaultObject);
			if (!bIsClassDefaultObject)
			{
				const int32 NumberOfMaterial = Component->GetNumMaterials();
				for (int32 Index = 0; Index < NumberOfMaterial; ++Index)
				{
					if (Component->GetMaterial(Index) == MaterialToBeReplaced)
					{
						FEditPropertyChain PropertyChain;
						PropertyChain.AddHead(MaterialProperty);
						static_cast<UObject*>(Component)->PreEditChange(PropertyChain);

						// Set the material
						Component->SetMaterial(Index, NewMaterial);
						++NumberOfChanges;

						ObjectsThatChanged.Add(Component);
					}
				}
			}
		}

		// Route post edit change after all components have had their values changed.  This is to avoid
		// construction scripts from re-running in the middle of setting values and wiping out components we need to modify
		for (UObject* ObjectData : ObjectsThatChanged)
		{
			FPropertyChangedEvent PropertyEvent(MaterialProperty);
			ObjectData->PostEditChangeProperty(PropertyEvent);
		}

		return NumberOfChanges;
	}

	template<class TPrimitiveComponent>
	bool FindValidActorAndComponents(TArray<AStaticMeshActor*> ActorsToTest, TArray<AStaticMeshActor*>& OutValidActor, TArray<TPrimitiveComponent*>& OutPrimitiveComponent, FVector& OutAverageLocation, FString& OutFailureReason)
	{
		for (int32 Index = ActorsToTest.Num() - 1; Index >= 0; --Index)
		{
			if (!IsValid(ActorsToTest[Index]))
			{
				ActorsToTest.RemoveAtSwap(Index);
			}
		}

		if (ActorsToTest.Num() == 0)
		{
			return false;
		}

		// All actors need to come from the same World
		UWorld* CurrentWorld = ActorsToTest[0]->GetWorld();
		if (CurrentWorld == nullptr)
		{
			OutFailureReason = TEXT("The actors were not in a valid world.");
			return false;
		}
		if (CurrentWorld->WorldType != EWorldType::Editor && CurrentWorld->WorldType != EWorldType::EditorPreview)
		{
			OutFailureReason = TEXT("The actors were not in an editor world.");
			return false;
		}

		ULevel* CurrentLevel = ActorsToTest[0]->GetLevel();
		if (CurrentLevel == nullptr)
		{
			OutFailureReason = TEXT("The actors were not in a valid level.");
			return false;
		}

		FVector PivotLocation = FVector::ZeroVector;

		OutPrimitiveComponent.Reset(ActorsToTest.Num());
		OutValidActor.Reset(ActorsToTest.Num());
		{
			bool bShowedDifferentLevelMessage = false;
			for (AStaticMeshActor* MeshActor : ActorsToTest)
			{
				if (MeshActor->GetWorld() != CurrentWorld)
				{
					OutFailureReason = TEXT("Some actors were not from the same world.");
					return false;
				}

				if (!bShowedDifferentLevelMessage && MeshActor->GetLevel() != CurrentLevel)
				{
					UE_LOGF(LogStaticMeshEditorSubsystem, Log, "Not all actors are from the same level. The Actor will be created in the first level found.");
					bShowedDifferentLevelMessage = true;
				}

				PivotLocation += MeshActor->GetActorLocation();

				TInlineComponentArray<UStaticMeshComponent*> ComponentArray;
				MeshActor->GetComponents(ComponentArray);

				bool bActorIsValid = false;
				for (UStaticMeshComponent* MeshCmp : ComponentArray)
				{
					if (MeshCmp->GetStaticMesh() && MeshCmp->GetStaticMesh()->GetRenderData())
					{
						bActorIsValid = true;
						OutPrimitiveComponent.Add(MeshCmp);
					}
				}

				//Actor needs at least one StaticMeshComponent to be considered valid
				if (bActorIsValid)
				{
					OutValidActor.Add(MeshActor);
				}
			}
		}

		OutAverageLocation = PivotLocation / OutValidActor.Num();

		return true;
	}

	FName GenerateValidOwnerBasedComponentNameForNewOwner(UStaticMeshComponent* OriginalComponent, AActor* NewOwner)
	{
		check(OriginalComponent);
		check(OriginalComponent->GetOwner());
		check(NewOwner);

		//Find first valid name on new owner by incrementing internal index
		FName NewName = OriginalComponent->GetOwner()->GetFName();
		const int32 InitialNumber = NewName.GetNumber();
		while (FindObjectFast<UObject>(NewOwner, NewName) != nullptr)
		{
			uint32 NextNumber = NewName.GetNumber();
			if (NextNumber >= 0xfffffe)
			{
				NewName = NAME_None;
				break;
			}
			++NextNumber;
			NewName.SetNumber(NextNumber);
		}

		return NewName;
	}
}

UStaticMeshEditorSubsystem::UStaticMeshEditorSubsystem()
	: UEditorSubsystem()
{

}

int32 UStaticMeshEditorSubsystem::SetLodsWithNotification(UStaticMesh* StaticMesh, const FStaticMeshReductionOptions& ReductionOptions, bool bApplyChanges)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return -1;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODs: The StaticMesh is null.");
		return -1;
	}

	// If LOD 0 does not exist, warn and return
	if (StaticMesh->GetNumSourceModels() == 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODs: This StaticMesh does not have LOD 0.");
		return -1;
	}

	if (ReductionOptions.ReductionSettings.Num() == 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODs: Nothing done as no LOD settings were provided.");
		return -1;
	}

	// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
	bool bStaticMeshIsEdited = false;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	if (bApplyChanges)
	{
		StaticMesh->Modify();
	}

	// Resize array of LODs to only keep LOD 0
	StaticMesh->SetNumSourceModels(1);

	// Set up LOD 0
	StaticMesh->GetSourceModel(0).ReductionSettings.PercentTriangles = ReductionOptions.ReductionSettings[0].PercentTriangles;
	StaticMesh->GetSourceModel(0).ScreenSize = ReductionOptions.ReductionSettings[0].ScreenSize;

	int32 LODIndex = 1;
	for (; LODIndex < ReductionOptions.ReductionSettings.Num(); ++LODIndex)
	{
		// Create new SourceModel for new LOD
		FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();

		// Copy settings from previous LOD
		SrcModel.BuildSettings = StaticMesh->GetSourceModel(LODIndex - 1).BuildSettings;
		SrcModel.ReductionSettings = StaticMesh->GetSourceModel(LODIndex - 1).ReductionSettings;

		// Modify reduction settings based on user's requirements
		SrcModel.ReductionSettings.PercentTriangles = ReductionOptions.ReductionSettings[LODIndex].PercentTriangles;
		SrcModel.ScreenSize = ReductionOptions.ReductionSettings[LODIndex].ScreenSize;

		// Stop when reaching maximum of supported LODs
		if (StaticMesh->GetNumSourceModels() == MAX_STATIC_MESH_LODS)
		{
			break;
		}
	}

	StaticMesh->SetAutoComputeLODScreenSize(ReductionOptions.bAutoComputeLODScreenSize ? 1 : 0);

	if (bApplyChanges)
	{
		// Request re-building of mesh with new LODs
		StaticMesh->PostEditChange();

		// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
		if (bStaticMeshIsEdited)
		{
			AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
		}
	}

	return LODIndex;
}

void UStaticMeshEditorSubsystem::GetLodReductionSettings(const UStaticMesh* StaticMesh, const int32 LodIndex, FMeshReductionSettings& OutReductionOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLodReductionSettings: The StaticMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || StaticMesh->GetNumSourceModels() <= LodIndex)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLodReductionSettings: Invalid LOD index.");
		return;
	}

	const FStaticMeshSourceModel& LODModel = StaticMesh->GetSourceModel(LodIndex);

	// Copy over the reduction settings
	OutReductionOptions = LODModel.ReductionSettings;
}

void UStaticMeshEditorSubsystem::SetLodReductionSettings(UStaticMesh* StaticMesh, const int32 LodIndex, const FMeshReductionSettings& ReductionOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodReductionSettings: The StaticMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || StaticMesh->GetNumSourceModels() <= LodIndex)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodReductionSettings: Invalid LOD index.");
		return;
	}

	// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
	bool bStaticMeshIsEdited = false;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	StaticMesh->Modify();

	FStaticMeshSourceModel& LODModel = StaticMesh->GetSourceModel(LodIndex);

	// Copy over the reduction settings
	LODModel.ReductionSettings = ReductionOptions;

	// Request re-building of mesh with new LODs
	StaticMesh->PostEditChange();

	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	if (bStaticMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
	}
}

void UStaticMeshEditorSubsystem::GetLodBuildSettings(const UStaticMesh* StaticMesh, const int32 LodIndex, FMeshBuildSettings& OutBuildOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLodBuildSettings: The StaticMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || StaticMesh->GetNumSourceModels() <= LodIndex)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLodBuildSettings: Invalid LOD index.");
		return;
	}

	const FStaticMeshSourceModel& LODModel = StaticMesh->GetSourceModel(LodIndex);

	// Copy over the reduction settings
	OutBuildOptions = LODModel.BuildSettings;
}

void UStaticMeshEditorSubsystem::SetLodBuildSettings(UStaticMesh* StaticMesh, const int32 LodIndex, const FMeshBuildSettings& BuildOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodBuildSettings: The StaticMesh is null.");
		return;
	}

	// If LOD 0 does not exist, warn and return
	if (LodIndex < 0 || StaticMesh->GetNumSourceModels() <= LodIndex)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodBuildSettings: Invalid LOD index.");
		return;
	}

	// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
	bool bStaticMeshIsEdited = false;
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	StaticMesh->Modify();

	FStaticMeshSourceModel& LODModel = StaticMesh->GetSourceModel(LodIndex);

	// Copy over the build settings
	LODModel.BuildSettings = BuildOptions;

	// Request re-building of mesh with new LODs
	StaticMesh->PostEditChange();

	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	if (bStaticMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
	}
}

FName UStaticMeshEditorSubsystem::GetLODGroup(const UStaticMesh* StaticMesh)
{
	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh GetLODGroup: The StaticMesh is null.");
		return NAME_None;
	}

	return StaticMesh->GetLODGroup();
}

bool UStaticMeshEditorSubsystem::SetLODGroup(UStaticMesh* StaticMesh, FName LODGroup, bool bRebuildImmediately)
{
	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh SetLODGroup: The StaticMesh is null.");
		return false;
	}

	TArray<FName> LODGroups;
	UStaticMesh::GetLODGroups(LODGroups);
	
	if(LODGroups.Contains(LODGroup))
	{
		GWarn->BeginSlowTask(FText::Format(FText::FromString("SetLODGroup: Applying changes to %s"), FText::FromString(StaticMesh->GetName())), true, false);
		// Close the mesh editor to prevent crashing. If changes are applied, reopen it after the mesh has been built.
		
		bool bStaticMeshIsEdited = false;
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
			bStaticMeshIsEdited = true;
		}
		
		StaticMesh->SetLODGroup(LODGroup, bRebuildImmediately);

		GWarn->EndSlowTask();

		return true;
	}
	UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODGroup: %ls is not a valid LODGroup", *LODGroup.ToString());
	return false;
		
}

int32 UStaticMeshEditorSubsystem::ImportLOD(UStaticMesh* BaseStaticMesh, const int32 LODIndex, const FString& SourceFilename)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ImportLOD: Cannot import or re-import when editor PIE is active.");
		return INDEX_NONE;
	}

	if (BaseStaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ImportLOD: The StaticMesh is null.");
		return INDEX_NONE;
	}

	// Make sure the LODIndex we want to add the LOD is valid
	if (BaseStaticMesh->GetNumSourceModels() < LODIndex)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ImportLOD: Invalid LODIndex, the LOD index cannot be greater the the number of LOD, static mesh cannot have hole in the LOD array.");
		return INDEX_NONE;
	}

	FString ResolveFilename = SourceFilename;
	const bool bSourceFileExists = FPaths::FileExists(ResolveFilename);
	if (!bSourceFileExists)
	{
		if (BaseStaticMesh->IsSourceModelValid(LODIndex))
		{
			const FStaticMeshSourceModel& SourceModel = BaseStaticMesh->GetSourceModel(LODIndex);
			ResolveFilename = SourceModel.SourceImportFilename.IsEmpty() ?
				SourceModel.SourceImportFilename :
				UAssetImportData::ResolveImportFilename(SourceModel.SourceImportFilename, nullptr);
		}
	}

	if (!FPaths::FileExists(ResolveFilename))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ImportLOD: Invalid source filename.");
		return INDEX_NONE;
	}

	constexpr bool bAsyncFalse = false;
	if (!FbxMeshUtils::ImportStaticMeshLOD(BaseStaticMesh, ResolveFilename, LODIndex, bAsyncFalse))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ImportLOD: Cannot import mesh LOD.");
		return INDEX_NONE;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostLODImport(BaseStaticMesh, LODIndex);

	return LODIndex;
}

bool UStaticMeshEditorSubsystem::ReimportAllCustomLODs(UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ReimportAllCustomLODs: Cannot import or re-import when editor PIE is active.");
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ReimportAllCustomLODs: The StaticMesh is null.");
		return false;
	}

	bool bResult = true;
	int32 LODNumber = StaticMesh->GetNumLODs();
	//Iterate the static mesh LODs, start at index 1
	for (int32 LODIndex = 1; LODIndex < LODNumber; ++LODIndex)
	{
		const FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LODIndex);
		//Skip LOD import in the same file as the base mesh, they are already re-import
		if (SourceModel.bImportWithBaseMesh)
		{
			continue;
		}

		bool bHasBeenSimplified = !StaticMesh->IsMeshDescriptionValid(LODIndex) || StaticMesh->IsReductionActive(LODIndex);
		if (bHasBeenSimplified)
		{
			continue;
		}

		if (ImportLOD(StaticMesh, LODIndex, SourceModel.SourceImportFilename) != LODIndex)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "StaticMesh ReimportAllCustomLODs: Cannot re-import LOD %d.", LODIndex);
			bResult = false;
		}
	}
	return bResult;
}

int32 UStaticMeshEditorSubsystem::SetLodFromStaticMesh(UStaticMesh* DestinationStaticMesh, int32 DestinationLodIndex, UStaticMesh* SourceStaticMesh, int32 SourceLodIndex, bool bReuseExistingMaterialSlots)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return -1;
	}


	if (DestinationStaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodFromStaticMesh: The DestinationStaticMesh is null.");
		return -1;
	}

	if (SourceStaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodFromStaticMesh: The SourceStaticMesh is null.");
		return -1;
	}

	if (!SourceStaticMesh->IsSourceModelValid(SourceLodIndex))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodFromStaticMesh: SourceLodIndex is invalid.");
		return -1;
	}

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(DestinationStaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(DestinationStaticMesh);
		bStaticMeshIsEdited = true;
	}

	DestinationStaticMesh->Modify();

	if (DestinationStaticMesh->GetNumSourceModels() < DestinationLodIndex + 1)
	{
		// Add one LOD
		DestinationStaticMesh->AddSourceModel();

		DestinationLodIndex = DestinationStaticMesh->GetNumSourceModels() - 1;

		// The newly added SourceModel won't have a MeshDescription so create it explicitly
		DestinationStaticMesh->CreateMeshDescription(DestinationLodIndex);
	}

	// Transfers the build settings and the reduction settings.
	const FStaticMeshSourceModel& SourceMeshSourceModel = SourceStaticMesh->GetSourceModel(SourceLodIndex);
	FStaticMeshSourceModel& DestinationMeshSourceModel = DestinationStaticMesh->GetSourceModel(DestinationLodIndex);
	DestinationMeshSourceModel.BuildSettings = SourceMeshSourceModel.BuildSettings;
	DestinationMeshSourceModel.ReductionSettings = SourceMeshSourceModel.ReductionSettings;
	// Base the reduction on the new lod
	DestinationMeshSourceModel.ReductionSettings.BaseLODModel = DestinationLodIndex;

	bool bDoesSourceLodUseReduction = SourceStaticMesh->IsReductionActive(SourceLodIndex);


	int32 BaseSourceLodIndex = bDoesSourceLodUseReduction ? SourceMeshSourceModel.ReductionSettings.BaseLODModel : SourceLodIndex;
	bool bIsReductionSettingAproximated = false;

	// Find the original mesh description for this LOD
	while (!SourceStaticMesh->IsMeshDescriptionValid(BaseSourceLodIndex))
	{
		if (!SourceStaticMesh->IsSourceModelValid(BaseSourceLodIndex))
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodFromStaticMesh: The SourceStaticMesh is in a invalid state.");
			return -1;
		}

		const FMeshReductionSettings& PossibleSourceMeshReductionSetting = SourceStaticMesh->GetSourceModel(BaseSourceLodIndex).ReductionSettings;
		DestinationMeshSourceModel.ReductionSettings.PercentTriangles *= PossibleSourceMeshReductionSetting.PercentTriangles;
		DestinationMeshSourceModel.ReductionSettings.PercentVertices *= PossibleSourceMeshReductionSetting.PercentVertices;
		BaseSourceLodIndex = SourceStaticMesh->GetSourceModel(BaseSourceLodIndex).ReductionSettings.BaseLODModel;

		bIsReductionSettingAproximated = true;
	}

	if (bIsReductionSettingAproximated)
	{
		TArray<FStringFormatArg> InOrderedArguments;
		InOrderedArguments.Reserve(4);
		InOrderedArguments.Add(SourceStaticMesh->GetName());
		InOrderedArguments.Add(SourceLodIndex);
		InOrderedArguments.Add(DestinationLodIndex);
		InOrderedArguments.Add(DestinationStaticMesh->GetName());

		UE_LOGF(LogStaticMeshEditorSubsystem, Warning, "%ls", *FString::Format(TEXT("SetLodFromStaticMesh: The reduction settings from the SourceStaticMesh {0} LOD {1} were approximated."
			" The LOD {2} from {3} might not be identical."), InOrderedArguments));
	}

	// Copy the source import file.
	DestinationMeshSourceModel.SourceImportFilename = SourceStaticMesh->GetSourceModel(BaseSourceLodIndex).SourceImportFilename;

	// Copy the mesh description
	const FMeshDescription& SourceMeshDescription = *SourceStaticMesh->GetMeshDescription(BaseSourceLodIndex);
	FMeshDescription& DestinationMeshDescription = *DestinationStaticMesh->GetMeshDescription(DestinationLodIndex);
	DestinationMeshDescription = SourceMeshDescription;
	DestinationStaticMesh->CommitMeshDescription(DestinationLodIndex);

	// Assign materials for the destination LOD
	{
		auto FindMaterialIndex = [](UStaticMesh* StaticMesh, const UMaterialInterface* Material) -> int32
		{
			for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++MaterialIndex)
			{
				if (StaticMesh->GetMaterial(MaterialIndex) == Material)
				{
					return MaterialIndex;
				}
			}

			return INDEX_NONE;
		};

		TMap< int32, int32 > LodSectionMaterialMapping; // LOD section index -> destination material index

		int32 NumDestinationMaterial = DestinationStaticMesh->GetStaticMaterials().Num();

		const int32 SourceLodNumSections = SourceStaticMesh->GetSectionInfoMap().GetSectionNumber(SourceLodIndex);

		for (int32 SourceLodSectionIndex = 0; SourceLodSectionIndex < SourceLodNumSections; ++SourceLodSectionIndex)
		{
			const FMeshSectionInfo& SourceMeshSectionInfo = SourceStaticMesh->GetSectionInfoMap().Get(SourceLodIndex, SourceLodSectionIndex);

			const UMaterialInterface* SourceMaterial = SourceStaticMesh->GetMaterial(SourceMeshSectionInfo.MaterialIndex);

			int32 DestinationMaterialIndex = INDEX_NONE;

			if (bReuseExistingMaterialSlots)
			{
				DestinationMaterialIndex = FindMaterialIndex(DestinationStaticMesh, SourceMaterial);
			}

			if (DestinationMaterialIndex == INDEX_NONE)
			{
				DestinationMaterialIndex = NumDestinationMaterial++;
			}

			LodSectionMaterialMapping.Add(SourceLodSectionIndex, DestinationMaterialIndex);
		}

		for (TMap< int32, int32 >::TConstIterator It = LodSectionMaterialMapping.CreateConstIterator(); It; ++It)
		{
			const int32 SectionIndex = It->Key;

			const FMeshSectionInfo& SourceSectionInfo = SourceStaticMesh->GetSectionInfoMap().Get(SourceLodIndex, SectionIndex);

			UMaterialInterface* SourceMaterial = SourceStaticMesh->GetMaterial(SourceSectionInfo.MaterialIndex);

			const int32 SourceMaterialIndex = SourceSectionInfo.MaterialIndex;
			const int32 DestinationMaterialIndex = It->Value;

			if (!DestinationStaticMesh->GetStaticMaterials().IsValidIndex(DestinationMaterialIndex))
			{
				DestinationStaticMesh->GetStaticMaterials().Add(SourceStaticMesh->GetStaticMaterials()[SourceSectionInfo.MaterialIndex]);

				ensure(DestinationStaticMesh->GetStaticMaterials().Num() == DestinationMaterialIndex + 1); // We assume that we are not creating holes in StaticMaterials
			}

			FMeshSectionInfo DestinationSectionInfo = SourceSectionInfo;
			DestinationSectionInfo.MaterialIndex = DestinationMaterialIndex;

			DestinationStaticMesh->GetSectionInfoMap().Set(DestinationLodIndex, SectionIndex, MoveTemp(DestinationSectionInfo));
		}
	}

	DestinationStaticMesh->PostEditChange();

	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	if (bStaticMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(DestinationStaticMesh);
	}

	return DestinationLodIndex;
}

int32 UStaticMeshEditorSubsystem::GetLodCount(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLODCount: The StaticMesh is null.");
		return -1;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return -1;
	}


	return StaticMesh->GetNumSourceModels();
}

bool UStaticMeshEditorSubsystem::RemoveLods(UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveLODs: The StaticMesh is null.");
		return false;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}


	// No main LOD, skip
	if (StaticMesh->GetNumSourceModels() == 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveLODs: This StaticMesh does not have LOD 0.");
		return false;
	}

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	// Reduce array of source models to 1
	StaticMesh->Modify();
	StaticMesh->SetNumSourceModels(1);

	// Request re-building of mesh with new LODs
	StaticMesh->PostEditChange();

	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	if (bStaticMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
	}

	return true;
}

TArray<float> UStaticMeshEditorSubsystem::GetLodScreenSizes(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	TArray<float> ScreenSizes;

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return ScreenSizes;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetLodScreenSizes: The StaticMesh is null.");
		return ScreenSizes;
	}

	for (int i = 0; i < StaticMesh->GetNumLODs(); i++)
	{
		if (StaticMesh->GetRenderData())
		{
			float CurScreenSize = StaticMesh->GetRenderData()->ScreenSize[i].GetValue();
			ScreenSizes.Add(CurScreenSize);
		}
		else
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Warning, "GetLodScreenSizes: The RenderData is invalid for LOD %d.", i);
		}
	}

	return ScreenSizes;

}

bool UStaticMeshEditorSubsystem::SetLodScreenSizes(UStaticMesh* StaticMesh, const TArray<float>& ScreenSizes)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodScreenSizes: Input StaticMesh is null.");
		return false;
	}

	FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
	if (RenderData == nullptr || (StaticMesh->GetNumLODs() == 0))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodScreenSizes: Input StaticMesh is invalid (missing RenderData or meshes).");
		return false;
	}

	if (ScreenSizes.Num() == 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLodScreenSizes: Input ScreenSizes array is empty.");
		return false;
	}

	// If not enough screen sizes, we set remainder to arbitrary monotonically decreasing defaults, and also ensure consecutive
	// values are monotonically decreasing, similar to what the user interface does when editing the values manually.
	if (ScreenSizes.Num() < StaticMesh->GetNumLODs())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Warning, "SetLodScreenSizes: Only %d of %d ScreenSizes provided, remainder will be set to arbitrary defaults.",
			ScreenSizes.Num(), StaticMesh->GetNumLODs());
	}

	const float MonotonicDifference = 0.0001f;		// This difference value matches the user interface
	bool bSanitizationRequired = false;

	// Disable automatic screen size calculation, since we're providing manually overriden values
	StaticMesh->SetAutoComputeLODScreenSize(false);

	// Arbitrarily set this to a value that won't affect the monotonic clamping on the first iteration of the loop
	float LastScreenSize = ScreenSizes[0] + 2.0f * MonotonicDifference;

	for (int i = 0; i < StaticMesh->GetNumLODs(); i++)
	{
		float ScreenSizeForLOD;
		if (i < ScreenSizes.Num())
		{
			ScreenSizeForLOD = FMath::Min(ScreenSizes[i], LastScreenSize - MonotonicDifference);
		}
		else
		{
			ScreenSizeForLOD = LastScreenSize - MonotonicDifference;
		}

		ScreenSizeForLOD = FMath::Max(ScreenSizeForLOD, 0.0f);

		// Track if the input values needed to be sanitized in any way, so we can warn the user this happened.
		if (i < ScreenSizes.Num() && ScreenSizeForLOD != ScreenSizes[i])
		{
			bSanitizationRequired = true;
		}

		RenderData->ScreenSize[i].Default = ScreenSizeForLOD;
		StaticMesh->GetSourceModel(i).ScreenSize = ScreenSizeForLOD;

		LastScreenSize = ScreenSizeForLOD;
	}

	if (bSanitizationRequired)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Warning, "SetLodScreenSizes: Some input values were sanitized to be monotonic.");
	}

	return true;
}

bool UStaticMeshEditorSubsystem::ImportNaniteHiResMesh(UStaticMesh* StaticMesh, const FString& SourceFilename, bool bShowDialogWhenFileMissing)
{
	if (!StaticMesh)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "ImportNaniteHiResMesh: The StaticMesh is null.");
		return false;
	}
	FStaticMeshSourceModel& HiResModel = StaticMesh->GetHiResSourceModel();
	FString ResolveFilename = SourceFilename;
	if (ResolveFilename.IsEmpty())
	{
		//Grab the existing source
		ResolveFilename = HiResModel.SourceImportFilename;
	}

	bool bSourceFileExists = FPaths::FileExists(ResolveFilename);
	if (!bSourceFileExists)
	{
		ResolveFilename = ResolveFilename.IsEmpty() ?
			ResolveFilename :
			UAssetImportData::ResolveImportFilename(ResolveFilename, nullptr);
	}

	bSourceFileExists = FPaths::FileExists(ResolveFilename);

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	const bool OriginalEnableNaniteValue = StaticMesh->GetNaniteSettings().bEnabled;
	StaticMesh->GetNaniteSettings().bEnabled = true;
	bool bResult = true;
	constexpr bool bAsyncFalse = false;
	if(bSourceFileExists)
	{
		
		if (!FbxMeshUtils::ImportStaticMeshHiResSourceModel(StaticMesh, ResolveFilename, bAsyncFalse))
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "ImportNaniteHiResMesh: Cannot import hi res mesh.");
			bResult = false;
		}
	}
	else if (bShowDialogWhenFileMissing)
	{
		if (!FbxMeshUtils::ImportStaticMeshHiResSourceModelDialog(StaticMesh))
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "ImportNaniteHiResMesh: Cannot import hi res mesh with dialog.");
			bResult = false;
		}
	}
	else
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "ImportNaniteHiResMesh: Cannot import hi res mesh, the provided file is invalid.");
		bResult = false;
	}
	
	if (!bResult)
	{
		StaticMesh->GetNaniteSettings().bEnabled = OriginalEnableNaniteValue;
	}

	if (bStaticMeshIsEdited)
	{
		AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
	}

	return bResult;
}

FString UStaticMeshEditorSubsystem::GetNaniteSourceFilename(const UStaticMesh* StaticMesh)
{
	FString ResolveFilename;
	if (!StaticMesh)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetNaniteSourceFilename: The StaticMesh is null.");
		return ResolveFilename;
	}
	const FStaticMeshSourceModel& HiResModel = StaticMesh->GetHiResSourceModel();
	ResolveFilename = HiResModel.SourceImportFilename;
	bool bSourceFileExists = FPaths::FileExists(ResolveFilename);
	if (!bSourceFileExists)
	{
		ResolveFilename = ResolveFilename.IsEmpty() ?
			ResolveFilename :
			UAssetImportData::ResolveImportFilename(ResolveFilename, nullptr);
	}
	return ResolveFilename;
}

bool UStaticMeshEditorSubsystem::UpdateNaniteSourceFilename(UStaticMesh* StaticMesh, const FString& NewSourceFilename)
{
	bool bResult = false;
	FString ResolveFilename = NewSourceFilename;
	if (!StaticMesh)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "UpdateNaniteSourceFilename: The StaticMesh is null.");
		return bResult;
	}
	if (!FPaths::FileExists(ResolveFilename))
	{
		ResolveFilename = ResolveFilename.IsEmpty() ?
			ResolveFilename :
			UAssetImportData::ResolveImportFilename(ResolveFilename, nullptr);
	}
	if (!StaticMesh->GetHiResSourceModel().SourceImportFilename.Equals(ResolveFilename))
	{
		if (ResolveFilename.IsEmpty())
		{
			FbxMeshUtils::RemoveStaticMeshHiRes(StaticMesh);
			bResult = true;
		}
		else if(FPaths::FileExists(ResolveFilename))
		{
			//We have a valid path, update it and re-import the nanite mesh
			StaticMesh->GetHiResSourceModel().SourceImportFilename = ResolveFilename;
			ImportNaniteHiResMesh(StaticMesh, ResolveFilename, false);
			bResult = true;
		}
		else
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "UpdateNaniteSourceFilename: The new source filename is invalid (it can be empty or must point to an existing filename).");
		}
	}
	return bResult;
}

FMeshNaniteSettings UStaticMeshEditorSubsystem::GetNaniteSettings(const UStaticMesh* StaticMesh)
{
	if (StaticMesh)
	{
		return StaticMesh->GetNaniteSettings();
	}
	else
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetNaniteSettings without a static mesh"), ELogVerbosity::Error);
		return FMeshNaniteSettings();
	}
}

void UStaticMeshEditorSubsystem::SetNaniteSettings(UStaticMesh* StaticMesh, FMeshNaniteSettings NaniteSettings, bool bApplyChanges)
{
	if (!StaticMesh)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetNaniteSettings without a static mesh"), ELogVerbosity::Error);
		return;
	}

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}
	
	StaticMesh->Modify();
	StaticMesh->SetNaniteSettings(NaniteSettings);

	if (bApplyChanges)
	{
		// Request re-building of mesh with new collision shapes
		StaticMesh->PostEditChange();

		// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
		if (bStaticMeshIsEdited)
		{
			AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
		}
	}
}

int32 UStaticMeshEditorSubsystem::AddSimpleCollisionsWithNotification(UStaticMesh* StaticMesh, const EScriptCollisionShapeType ShapeType, bool bApplyChanges)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "AddSimpleCollisions: The StaticMesh is null.");
		return INDEX_NONE;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return INDEX_NONE;
	}

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	int32 PrimIndex = INDEX_NONE;

	switch (ShapeType)
	{
	case EScriptCollisionShapeType::Box:
	{
		PrimIndex = GenerateBoxAsSimpleCollision(StaticMesh);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.BoxElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::Sphere:
	{
		PrimIndex = GenerateSphereAsSimpleCollision(StaticMesh);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.SphereElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::Capsule:
	{
		PrimIndex = GenerateSphylAsSimpleCollision(StaticMesh);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.SphylElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::NDOP10_X:
	{
		TArray<FVector>	DirArray(KDopDir10X, 10);
		PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::NDOP10_Y:
	{
		TArray<FVector>	DirArray(KDopDir10Y, 10);
		PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::NDOP10_Z:
	{
		TArray<FVector>	DirArray(KDopDir10Z, 10);
		PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::NDOP18:
	{
		TArray<FVector>	DirArray(KDopDir18, 18);
		PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	case EScriptCollisionShapeType::NDOP26:
	{
		TArray<FVector>	DirArray(KDopDir26, 26);
		PrimIndex = GenerateKDopAsSimpleCollision(StaticMesh, DirArray);
		if(PrimIndex != INDEX_NONE)
		{
			StaticMesh->GetBodySetup()->AggGeom.ConvexElems[PrimIndex].bIsGenerated = true;
		}
		break;
	}
	}

	if (bApplyChanges)
	{
		// Request re-building of mesh with new collision shapes
		StaticMesh->PostEditChange();

		// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
		if (bStaticMeshIsEdited)
		{
			AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
		}
	}

	return PrimIndex;
}

int32 UStaticMeshEditorSubsystem::GetSimpleCollisionCount(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetSimpleCollisionCount: The StaticMesh is null.");
		return -1;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return -1;
	}

	UBodySetup* BodySetup = StaticMesh->GetBodySetup();
	if (BodySetup == nullptr)
	{
		return 0;
	}

	int32 Count = BodySetup->AggGeom.BoxElems.Num();
	Count += BodySetup->AggGeom.SphereElems.Num();
	Count += BodySetup->AggGeom.SphylElems.Num();

	return Count;
}

TEnumAsByte<ECollisionTraceFlag> UStaticMeshEditorSubsystem::GetCollisionComplexity(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetCollisionComplexity: The StaticMesh is null.");
		return ECollisionTraceFlag::CTF_UseDefault;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return ECollisionTraceFlag::CTF_UseDefault;
	}

	if (StaticMesh->GetBodySetup())
	{
		return StaticMesh->GetBodySetup()->CollisionTraceFlag;
	}

	return ECollisionTraceFlag::CTF_UseDefault;
}

int32 UStaticMeshEditorSubsystem::GetConvexCollisionCount(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetConvexCollisionCount: The StaticMesh is null.");
		return -1;
	}

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return -1;
	}

	UBodySetup* BodySetup = StaticMesh->GetBodySetup();
	if (BodySetup == nullptr)
	{
		return 0;
	}

	return BodySetup->AggGeom.ConvexElems.Num();
}

bool UStaticMeshEditorSubsystem::BulkSetConvexDecompositionCollisionsWithNotification(const TArray<UStaticMesh*>& InStaticMeshes, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, bool bApplyChanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStaticMeshEditorSubsystem::SetConvexDecompositionCollisionsWithNotification)

	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	TArray<UStaticMesh*> StaticMeshes(InStaticMeshes);
	StaticMeshes.RemoveAll([](const UStaticMesh* StaticMesh) { return StaticMesh == nullptr || !StaticMesh->IsMeshDescriptionValid(0); });

	if (StaticMeshes.Num() == 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetConvexDecompositionCollisions: The StaticMesh is null.");
		return false;
	}

	if (HullCount < 0 || HullPrecision < 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetConvexDecompositionCollisions: Parameters HullCount and HullPrecision must be positive.");
		return false;
	}

	if (Algo::AnyOf(StaticMeshes, [](const UStaticMesh* StaticMesh) { return StaticMesh->GetRenderData() == nullptr; }))
	{
		UStaticMesh::BatchBuild(StaticMeshes);
	}

	Algo::SortBy(
		StaticMeshes,
		[](const UStaticMesh* StaticMesh) { return StaticMesh->GetRenderData()->LODResources[0].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices(); },
		TGreater<>()
	);

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

	TSet<UStaticMesh*> EditedStaticMeshes;
	for (UStaticMesh* StaticMesh : StaticMeshes)
	{
		if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
			EditedStaticMeshes.Add(StaticMesh);
		}

		if (StaticMesh->GetBodySetup())
		{
			if (bApplyChanges)
			{
				StaticMesh->GetBodySetup()->Modify();
			}

			// Remove simple collisions
			StaticMesh->GetBodySetup()->RemoveSimpleCollision();
		}
	}

	TArray<bool> bResults;
	bResults.SetNumZeroed(StaticMeshes.Num());

	TAtomic<uint32> Processed(0);
	TFuture<void> Result =
		Async(
			EAsyncExecution::ThreadPool,
			[&Processed, &bResults, &StaticMeshes, HullCount, MaxHullVerts, HullPrecision]()
			{
				ParallelFor(
					StaticMeshes.Num(),
					[&Processed, &bResults, &StaticMeshes, HullCount, MaxHullVerts, HullPrecision](int32 Index)
					{
						bResults[Index] = InternalEditorMeshLibrary::GenerateConvexCollision(StaticMeshes[Index], HullCount, MaxHullVerts, HullPrecision);
						Processed++;
					},
					EParallelForFlags::Unbalanced
						);
			}
	);

	uint32 LastProcessed = 0;
	const FText ProgressText = LOCTEXT("ComputingConvexCollision", "Computing convex collision for static mesh {0}/{1} ...");
	FScopedSlowTask Progress(static_cast<float>(StaticMeshes.Num()), FText::Format(ProgressText, LastProcessed, StaticMeshes.Num()));
	Progress.MakeDialog();

	while (!Result.WaitFor(FTimespan::FromMilliseconds(33.0)))
	{
		uint32 LocalProcessed = Processed.Load(EMemoryOrder::Relaxed);
		Progress.EnterProgressFrame(static_cast<float>(LocalProcessed - LastProcessed), FText::Format(ProgressText, LocalProcessed, StaticMeshes.Num()));
		LastProcessed = LocalProcessed;
	}

	// refresh collision change back to static mesh components
	RefreshCollisionChanges(StaticMeshes);

	if (bApplyChanges)
	{
		for (UStaticMesh* StaticMesh : StaticMeshes)
		{
			// Mark mesh as dirty
			StaticMesh->MarkPackageDirty();

			// Request re-building of mesh following collision changes
			StaticMesh->PostEditChange();
		}
	}

	// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
	for (UStaticMesh* StaticMesh : EditedStaticMeshes)
	{
		AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
	}

	return Algo::AllOf(bResults);
}

bool UStaticMeshEditorSubsystem::SetConvexDecompositionCollisionsWithNotification(UStaticMesh* StaticMesh, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, bool bApplyChanges)
{
	return BulkSetConvexDecompositionCollisionsWithNotification({ StaticMesh }, HullCount, MaxHullVerts, HullPrecision, bApplyChanges);
}

bool UStaticMeshEditorSubsystem::RemoveCollisionsWithNotification(UStaticMesh* StaticMesh, bool bApplyChanges)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveCollisions: The StaticMesh is null.");
		return false;
	}

	if (StaticMesh->GetBodySetup() == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Log, "RemoveCollisions: No collision set up. Nothing to do.");
		return true;
	}

	// Close the mesh editor to prevent crashing. Reopen it after the mesh has been built.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	bool bStaticMeshIsEdited = false;
	if (AssetEditorSubsystem->FindEditorForAsset(StaticMesh, false))
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(StaticMesh);
		bStaticMeshIsEdited = true;
	}

	if (bApplyChanges)
	{
		StaticMesh->GetBodySetup()->Modify();
	}

	// Remove simple collisions
	StaticMesh->GetBodySetup()->RemoveSimpleCollision();

	// refresh collision change back to static mesh components
	RefreshCollisionChange(*StaticMesh);

	if (bApplyChanges)
	{
		// Request re-building of mesh with new collision shapes
		StaticMesh->PostEditChange();

		// Reopen MeshEditor on this mesh if the MeshEditor was previously opened in it
		if (bStaticMeshIsEdited)
		{
			AssetEditorSubsystem->OpenEditorForAsset(StaticMesh);
		}
	}

	return true;
}

bool UStaticMeshEditorSubsystem::CanModifyStaticMeshSection(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex, const FString& LogFunctionName)
{
	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "%ls: The StaticMesh is null.", *LogFunctionName);
		return false;
	}

	if (LODIndex >= StaticMesh->GetNumLODs())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "%ls: Invalid LOD index %d (of %d).", *LogFunctionName, LODIndex, StaticMesh->GetNumLODs());
		return false;
	}

	if (SectionIndex >= StaticMesh->GetNumSections(LODIndex))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "%ls: Invalid section index %d (of %d).", *LogFunctionName, SectionIndex, StaticMesh->GetNumSections(LODIndex));
		return false;
	}

	return true;
}

void UStaticMeshEditorSubsystem::EnableSectionCastShadow(UStaticMesh* StaticMesh, bool bCastShadow, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "EnableSectionCastShadow"))
	{
		StaticMesh->Modify();

		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

		SectionInfo.bCastShadow = bCastShadow;

		StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

		StaticMesh->PostEditChange();
	}
}

bool UStaticMeshEditorSubsystem::IsSectionCastShadowEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);
	
	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "IsSectionCastShadowEnabled"))
	{
		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		return SectionInfo.bCastShadow;
	}

	return false;
}

void UStaticMeshEditorSubsystem::EnableSectionCollision(UStaticMesh* StaticMesh, bool bCollisionEnabled, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "EnableSectionCollision"))
	{
		StaticMesh->Modify();

		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

		SectionInfo.bEnableCollision = bCollisionEnabled;

		StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

		StaticMesh->PostEditChange();
	}
}

bool UStaticMeshEditorSubsystem::IsSectionCollisionEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "IsSectionCollisionEnabled"))
	{
		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		return SectionInfo.bEnableCollision;
	}
	return false;
}

void UStaticMeshEditorSubsystem::EnableSectionVisibleInRayTracing(UStaticMesh* StaticMesh, bool bVisibleInRayTracing, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "EnableSectionVisibleInRayTracing"))
	{
		StaticMesh->Modify();

		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

		SectionInfo.bVisibleInRayTracing = bVisibleInRayTracing;

		StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

		StaticMesh->PostEditChange();
	}
}

bool UStaticMeshEditorSubsystem::IsSectionVisibleInRayTracingEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "IsSectionVisibleInRayTracingEnabled"))
	{
		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		return SectionInfo.bVisibleInRayTracing;
	}
	return false;
}

void UStaticMeshEditorSubsystem::EnableSectionAffectDistanceFieldLighting(UStaticMesh* StaticMesh, bool bAffectDistanceFieldLighting, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "EnableSectionAffectDistanceFieldLighting"))
	{
		StaticMesh->Modify();

		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

		SectionInfo.bAffectDistanceFieldLighting = bAffectDistanceFieldLighting;

		StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

		StaticMesh->PostEditChange();
	}
}

bool UStaticMeshEditorSubsystem::IsSectionAffectDistanceFieldLightingEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "IsSectionAffectDistanceFieldLightingEnabled"))
	{
		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		return SectionInfo.bAffectDistanceFieldLighting;
	}
	return false;
}

void UStaticMeshEditorSubsystem::EnableSectionForceOpaque(UStaticMesh* StaticMesh, bool bForceOpaque, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "EnableSectionForceOpaque"))
	{
		StaticMesh->Modify();

		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

		SectionInfo.bForceOpaque = bForceOpaque;

		StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

		StaticMesh->PostEditChange();
	}
}

bool UStaticMeshEditorSubsystem::IsSectionForceOpaqueEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (CanModifyStaticMeshSection(StaticMesh, LODIndex, SectionIndex, "IsSectionForceOpaqueEnabled"))
	{
		FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
		return SectionInfo.bForceOpaque;
	}
	return false;
}

void UStaticMeshEditorSubsystem::SetLODMaterialSlot(UStaticMesh* StaticMesh, int32 MaterialSlotIndex, int32 LODIndex, int32 SectionIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODMaterialSlot: The StaticMesh is null.");
		return;
	}

	if (LODIndex >= StaticMesh->GetNumLODs())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODMaterialSlot: Invalid LOD index %d (of %d).", LODIndex, StaticMesh->GetNumLODs());
		return;
	}

	if (SectionIndex >= StaticMesh->GetNumSections(LODIndex))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODMaterialSlot: Invalid section index %d (of %d).", SectionIndex, StaticMesh->GetNumSections(LODIndex));
		return;
	}

	if (MaterialSlotIndex >= StaticMesh->GetStaticMaterials().Num())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetLODMaterialSlot: Invalid slot index %d (of %d).", MaterialSlotIndex, StaticMesh->GetStaticMaterials().Num());
		return;
	}

	StaticMesh->Modify();

	FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);

	SectionInfo.MaterialIndex = MaterialSlotIndex;

	StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);

	StaticMesh->PostEditChange();
}



int32 UStaticMeshEditorSubsystem::GetLODMaterialSlot(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex )
{
	TGuardValue<bool> UnattendedScriptGuard( GIsRunningUnattendedScript, true );

	if ( !EditorScriptingHelpers::CheckIfInEditorAndPIE() )
	{
		return INDEX_NONE;
	}

	if ( StaticMesh == nullptr )
	{
		UE_LOGF( LogStaticMeshEditorSubsystem, Error, "GetLODMaterialSlot: The StaticMesh is null." );
		return INDEX_NONE;
	}

	if ( LODIndex >= StaticMesh->GetNumLODs() )
	{
		UE_LOGF( LogStaticMeshEditorSubsystem, Error, "GetLODMaterialSlot: Invalid LOD index %d (of %d).", LODIndex, StaticMesh->GetNumLODs() );
		return INDEX_NONE;
	}

	if ( SectionIndex >= StaticMesh->GetNumSections( LODIndex ) )
	{
		UE_LOGF( LogStaticMeshEditorSubsystem, Error, "GetLODMaterialSlot: Invalid section index %d (of %d).", SectionIndex, StaticMesh->GetNumSections( LODIndex ) );
		return INDEX_NONE;
	}

	return StaticMesh->GetSectionInfoMap().Get( LODIndex, SectionIndex ).MaterialIndex;
}

bool UStaticMeshEditorSubsystem::HasVertexColors(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "HasVertexColors: The StaticMesh is null.");
		return false;
	}

	for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex)
	{
		const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LodIndex);
		FStaticMeshConstAttributes Attributes(*MeshDescription);
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		if (!VertexInstanceColors.IsValid())
		{
			continue;
		}

		for (const FVertexInstanceID VertexInstanceID : MeshDescription->VertexInstances().GetElementIDs())
		{
			FLinearColor VertexInstanceColor(VertexInstanceColors[VertexInstanceID]);
			if (VertexInstanceColor != FLinearColor::White)
			{
				return true;
			}
		}
	}
	return false;
}

bool UStaticMeshEditorSubsystem::HasInstanceVertexColors(const UStaticMeshComponent* StaticMeshComponent)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMeshComponent == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "HasInstanceVertexColors: The StaticMeshComponent is null.");
		return false;
	}

	for (const FStaticMeshComponentLODInfo& CurrentLODInfo : StaticMeshComponent->LODData)
	{
		if (CurrentLODInfo.OverrideVertexColors != nullptr || CurrentLODInfo.PaintedVertices.Num() > 0)
		{
			return true;
		}
	}

	return false;
}

bool UStaticMeshEditorSubsystem::SetGenerateLightmapUVs(UStaticMesh* StaticMesh, bool bGenerateLightmapUVs)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetGenerateLightmapUVs: The StaticMesh is null.");
		return false;
	}

	bool AnySettingsToChange = false;
	for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); ++LodIndex)
	{
		FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(LodIndex);
		//Make sure LOD is not a reduction before considering its BuildSettings
		if (StaticMesh->IsMeshDescriptionValid(LodIndex))
		{
			AnySettingsToChange = (SourceModel.BuildSettings.bGenerateLightmapUVs != bGenerateLightmapUVs);

			if (AnySettingsToChange)
			{
				break;
			}
		}
	}

	if (AnySettingsToChange)
	{
		StaticMesh->Modify();
		for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumSourceModels(); LodIndex++)
		{
			StaticMesh->GetSourceModel(LodIndex).BuildSettings.bGenerateLightmapUVs = bGenerateLightmapUVs;
		}

		StaticMesh->Build();
		StaticMesh->PostEditChange();
		return true;
	}

	return false;
}

int32 UStaticMeshEditorSubsystem::GetNumberVerts(const UStaticMesh* StaticMesh, int32 LODIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return 0;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetNumberVerts: The StaticMesh is null.");
		return 0;
	}

	return StaticMesh->GetNumVertices(LODIndex);
}

int32 UStaticMeshEditorSubsystem::GetNumberMaterials(const UStaticMesh* StaticMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return 0;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetNumberMaterials: The StaticMesh is null.");
		return 0;
	}

	return StaticMesh->GetStaticMaterials().Num();
}

void UStaticMeshEditorSubsystem::SetAllowCPUAccess(UStaticMesh* StaticMesh, bool bAllowCPUAccess)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "SetAllowCPUAccess: The StaticMesh is null.");
		return;
	}

	StaticMesh->Modify();
	StaticMesh->bAllowCPUAccess = bAllowCPUAccess;
	StaticMesh->PostEditChange();
}

int32 UStaticMeshEditorSubsystem::GetNumUVChannels(const UStaticMesh* StaticMesh, int32 LODIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return 0;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetNumUVChannels: The StaticMesh is null.");
		return 0;
	}

	if (LODIndex >= StaticMesh->GetNumLODs() || LODIndex < 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "GetNumUVChannels: The StaticMesh doesn't have LOD %d.", LODIndex);
		return 0;
	}

	return StaticMesh->GetNumUVChannels(LODIndex);
}

bool UStaticMeshEditorSubsystem::AddUVChannel(UStaticMesh* StaticMesh, int32 LODIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "AddUVChannel: The StaticMesh is null.");
		return false;
	}

	if (LODIndex >= StaticMesh->GetNumLODs() || LODIndex < 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "AddUVChannel: The StaticMesh doesn't have LOD %d.", LODIndex);
		return false;
	}

	if (StaticMesh->GetNumUVChannels(LODIndex) >= MAX_MESH_TEXTURE_COORDS_MD)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "AddUVChannel: Cannot add UV channel. Maximum number of UV channels reached (%d).", MAX_MESH_TEXTURE_COORDS_MD);
		return false;
	}

	return StaticMesh->AddUVChannel(LODIndex);
}

bool UStaticMeshEditorSubsystem::InsertUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "InsertUVChannel: The StaticMesh is null.");
		return false;
	}

	if (LODIndex >= StaticMesh->GetNumLODs() || LODIndex < 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "InsertUVChannel: The StaticMesh doesn't have LOD %d.", LODIndex);
		return false;
	}

	int32 NumUVChannels = StaticMesh->GetNumUVChannels(LODIndex);
	if (UVChannelIndex < 0 || UVChannelIndex > NumUVChannels)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "InsertUVChannel: Cannot insert UV channel. Given UV channel index %d is out of bounds.", UVChannelIndex);
		return false;
	}

	if (NumUVChannels >= MAX_MESH_TEXTURE_COORDS_MD)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "InsertUVChannel: Cannot add UV channel. Maximum number of UV channels reached (%d).", MAX_MESH_TEXTURE_COORDS_MD);
		return false;
	}

	return StaticMesh->InsertUVChannel(LODIndex, UVChannelIndex);
}

bool UStaticMeshEditorSubsystem::RemoveUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (StaticMesh == nullptr)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveUVChannel: The StaticMesh is null.");
		return false;
	}

	if (LODIndex >= StaticMesh->GetNumLODs() || LODIndex < 0)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveUVChannel: The StaticMesh doesn't have LOD %d.", LODIndex);
		return false;
	}

	int32 NumUVChannels = StaticMesh->GetNumUVChannels(LODIndex);
	if (NumUVChannels == 1)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveUVChannel: Cannot remove UV channel. There must be at least one channel.");
		return false;
	}

	if (UVChannelIndex < 0 || UVChannelIndex >= NumUVChannels)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "RemoveUVChannel: Cannot remove UV channel. Given UV channel index %d is out of bounds.", UVChannelIndex);
		return false;
	}

	return StaticMesh->RemoveUVChannel(LODIndex, UVChannelIndex);
}

bool UStaticMeshEditorSubsystem::GeneratePlanarUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector2D& Tiling)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (!InternalEditorMeshLibrary::IsUVChannelValid(StaticMesh, LODIndex, UVChannelIndex))
	{
		return false;
	}

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

	FUVMapParameters UVParameters(Position, Orientation.Quaternion(), StaticMesh->GetBoundingBox().GetSize(), FVector::OneVector, Tiling);

	TMap<FVertexInstanceID, FVector2D> TexCoords;
	FStaticMeshOperations::GeneratePlanarUV(*MeshDescription, UVParameters, TexCoords);

	return StaticMesh->SetUVChannel(LODIndex, UVChannelIndex, TexCoords);
}

bool UStaticMeshEditorSubsystem::GenerateCylindricalUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector2D& Tiling)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (!InternalEditorMeshLibrary::IsUVChannelValid(StaticMesh, LODIndex, UVChannelIndex))
	{
		return false;
	}

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

	FUVMapParameters UVParameters(Position, Orientation.Quaternion(), StaticMesh->GetBoundingBox().GetSize(), FVector::OneVector, Tiling);

	TMap<FVertexInstanceID, FVector2D> TexCoords;
	FStaticMeshOperations::GenerateCylindricalUV(*MeshDescription, UVParameters, TexCoords);

	return StaticMesh->SetUVChannel(LODIndex, UVChannelIndex, TexCoords);
}

bool UStaticMeshEditorSubsystem::GenerateBoxUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector& Size)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	if (!InternalEditorMeshLibrary::IsUVChannelValid(StaticMesh, LODIndex, UVChannelIndex))
	{
		return false;
	}

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);

	FUVMapParameters UVParameters(Position, Orientation.Quaternion(), Size, FVector::OneVector, FVector2D::UnitVector);

	TMap<FVertexInstanceID, FVector2D> TexCoords;
	FStaticMeshOperations::GenerateBoxUV(*MeshDescription, UVParameters, TexCoords);

	return StaticMesh->SetUVChannel(LODIndex, UVChannelIndex, TexCoords);
}

void UStaticMeshEditorSubsystem::ReplaceMeshComponentsMaterials(const TArray<UMeshComponent*>& MeshComponents, UMaterialInterface* MaterialToBeReplaced, UMaterialInterface* NewMaterial)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("ReplaceMeshComponentsMaterials", "Replace components materials"));

	int32 ChangeCounter = InternalEditorMeshLibrary::ReplaceMaterials(MeshComponents, MaterialToBeReplaced, NewMaterial);

	if (ChangeCounter > 0)
	{
		// Redraw viewports to reflect the material changes
		GEditor->RedrawLevelEditingViewports();
	}

	UE_LOGF(LogStaticMeshEditorSubsystem, Log, "ReplaceMeshComponentsMaterials. %d material(s) changed occurred.", ChangeCounter);
}

void UStaticMeshEditorSubsystem::ReplaceMeshComponentsMaterialsOnActors(const TArray<AActor*>& Actors, UMaterialInterface* MaterialToBeReplaced, UMaterialInterface* NewMaterial)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("ReplaceComponentUsedMaterial", "Replace components materials"));

	int32 ChangeCounter = 0;
	TInlineComponentArray<UMeshComponent*> ComponentArray;

	for (AActor* Actor : Actors)
	{
		if (Actor && IsValidChecked(Actor))
		{
			Actor->GetComponents(ComponentArray);
			ChangeCounter += InternalEditorMeshLibrary::ReplaceMaterials(ComponentArray, MaterialToBeReplaced, NewMaterial);
		}
	}

	if (ChangeCounter > 0)
	{
		// Redraw viewports to reflect the material changes
		GEditor->RedrawLevelEditingViewports();
	}

	UE_LOGF(LogStaticMeshEditorSubsystem, Log, "ReplaceMeshComponentsMaterialsOnActors. %d material(s) changed occurred.", ChangeCounter);
}


void UStaticMeshEditorSubsystem::ReplaceMeshComponentsMeshes(const TArray<UStaticMeshComponent*>& MeshComponents, UStaticMesh* MeshToBeReplaced, UStaticMesh* NewMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("ReplaceMeshComponentsMeshes", "Replace components meshes"));

	int32 ChangeCounter = InternalEditorMeshLibrary::ReplaceMeshes(MeshComponents, MeshToBeReplaced, NewMesh);

	if (ChangeCounter > 0)
	{
		// Redraw viewports to reflect the material changes
		GEditor->RedrawLevelEditingViewports();
	}

	UE_LOGF(LogStaticMeshEditorSubsystem, Log, "ReplaceMeshComponentsMeshes. %d mesh(es) changed occurred.", ChangeCounter);
}

void UStaticMeshEditorSubsystem::ReplaceMeshComponentsMeshesOnActors(const TArray<AActor*>& Actors, UStaticMesh* MeshToBeReplaced, UStaticMesh* NewMesh)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("ReplaceMeshComponentsMeshes", "Replace components meshes"));

	int32 ChangeCounter = 0;
	TInlineComponentArray<UStaticMeshComponent*> ComponentArray;

	for (AActor* Actor : Actors)
	{
		if (Actor && IsValidChecked(Actor))
		{
			Actor->GetComponents(ComponentArray);
			ChangeCounter += InternalEditorMeshLibrary::ReplaceMeshes(ComponentArray, MeshToBeReplaced, NewMesh);
		}
	}

	if (ChangeCounter > 0)
	{
		// Redraw viewports to reflect the material changes
		GEditor->RedrawLevelEditingViewports();
	}

	UE_LOGF(LogStaticMeshEditorSubsystem, Log, "ReplaceMeshComponentsMeshesOnActors. %d mesh(es) changed occurred.", ChangeCounter);
}

AActor* UStaticMeshEditorSubsystem::JoinStaticMeshActors(const TArray<AStaticMeshActor*>& ActorsToMerge, const FJoinStaticMeshActorsOptions& JoinOptions)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return nullptr;
	}

	TArray<AStaticMeshActor*> AllActors;
	TArray<UStaticMeshComponent*> AllComponents;
	FVector PivotLocation;
	FString FailureReason;
	if (!InternalEditorMeshLibrary::FindValidActorAndComponents(ActorsToMerge, AllActors, AllComponents, PivotLocation, FailureReason))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "JoinStaticMeshActors failed. %ls", *FailureReason);
		return nullptr;
	}

	if (AllActors.Num() < 2)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "JoinStaticMeshActors failed. A merge operation requires at least 2 valid Actors.");
		return nullptr;
	}

	// Create the new Actor
	FActorSpawnParameters Params;
	Params.OverrideLevel = AllActors[0]->GetLevel();
	AActor* NewActor = AllActors[0]->GetWorld()->SpawnActor<AActor>(PivotLocation, FRotator::ZeroRotator, Params);
	if (!NewActor)
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "JoinStaticMeshActors failed. Internal error while creating the join actor.");
		return nullptr;
	}

	if (!JoinOptions.NewActorLabel.IsEmpty())
	{
		NewActor->SetActorLabel(JoinOptions.NewActorLabel);
	}

	// Duplicate and attach all components to the new actors
	USceneComponent* NewRootComponent = NewObject<USceneComponent>(NewActor, TEXT("Root"));
	NewActor->SetRootComponent(NewRootComponent);
	NewRootComponent->SetMobility(EComponentMobility::Static);
	for (UStaticMeshComponent* ActorCmp : AllComponents)
	{
		FName NewName = NAME_None;
		if (JoinOptions.bRenameComponentsFromSource)
		{
			NewName = InternalEditorMeshLibrary::GenerateValidOwnerBasedComponentNameForNewOwner(ActorCmp, NewActor);
		}

		UStaticMeshComponent* NewComponent = DuplicateObject<UStaticMeshComponent>(ActorCmp, NewActor, NewName);
		NewActor->AddInstanceComponent(NewComponent);
		FTransform CmpTransform = ActorCmp->GetComponentToWorld();
		NewComponent->SetComponentToWorld(CmpTransform);
		NewComponent->AttachToComponent(NewRootComponent, FAttachmentTransformRules::KeepWorldTransform);
		NewComponent->RegisterComponent();
	}

	if (JoinOptions.bDestroySourceActors)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		UWorld* World = AllActors[0]->GetWorld();
		for (AActor* Actor : AllActors)
		{
			Layers->DisassociateActorFromLayers(Actor);
			World->EditorDestroyActor(Actor, true);
		}
	}

	//Select newly created actor
	GEditor->SelectNone(false, true, false);
	GEditor->SelectActor(NewActor, true, false);
	GEditor->NoteSelectionChange();

	UE_LOGF(LogStaticMeshEditorSubsystem, Log, "JoinStaticMeshActors joined %d actors toghether in actor '%ls'.", AllComponents.Num(), *NewActor->GetActorLabel());
	return NewActor;
}

bool UStaticMeshEditorSubsystem::MergeStaticMeshActors(const TArray<AStaticMeshActor*>& ActorsToMerge, const FMergeStaticMeshActorsOptions& MergeOptions, AStaticMeshActor*& OutMergedActor)
{
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	OutMergedActor = nullptr;

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	FString FailureReason;
	FString PackageName = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(MergeOptions.BasePackageName, FailureReason);
	if (PackageName.IsEmpty())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "MergeStaticMeshActors. Failed to convert the BasePackageName. %ls", *FailureReason);
		return false;
	}

	TArray<AStaticMeshActor*> AllActors;
	TArray<UPrimitiveComponent*> AllComponents;
	FVector PivotLocation;
	if (!InternalEditorMeshLibrary::FindValidActorAndComponents(ActorsToMerge, AllActors, AllComponents, PivotLocation, FailureReason))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "MergeStaticMeshActors failed. %ls", *FailureReason);
		return false;
	}

	//
	// See MeshMergingTool.cpp
	//
	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();


	FVector MergedActorLocation;
	TArray<UObject*> CreatedAssets;
	const float ScreenAreaSize = TNumericLimits<float>::Max();
	MeshUtilities.MergeComponentsToStaticMesh(AllComponents, AllActors[0]->GetWorld(), MergeOptions.MeshMergingSettings, nullptr, nullptr, PackageName, CreatedAssets, MergedActorLocation, ScreenAreaSize, true);

	UStaticMesh* MergedMesh = nullptr;
	if (!CreatedAssets.FindItemByClass(&MergedMesh))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "MergeStaticMeshActors failed. No mesh was created.");
		return false;
	}

	FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	for (UObject* Obj : CreatedAssets)
	{
		AssetRegistry.AssetCreated(Obj);
	}

	//Also notify the content browser that the new assets exists
	if (!IsRunningCommandlet())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(CreatedAssets, true);
	}

	// Place new mesh in the world
	if (MergeOptions.bSpawnMergedActor)
	{
		FActorSpawnParameters Params;
		Params.OverrideLevel = AllActors[0]->GetLevel();
		OutMergedActor = AllActors[0]->GetWorld()->SpawnActor<AStaticMeshActor>(MergedActorLocation, FRotator::ZeroRotator, Params);
		if (!OutMergedActor)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "MergeStaticMeshActors failed. Internal error while creating the merged actor.");
			return false;
		}

		OutMergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		OutMergedActor->SetActorLabel(MergeOptions.NewActorLabel);
		AllActors[0]->GetWorld()->UpdateCullDistanceVolumes(OutMergedActor, OutMergedActor->GetStaticMeshComponent());
	}

	// Remove source actors
	if (MergeOptions.bDestroySourceActors)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		UWorld* World = AllActors[0]->GetWorld();
		for (AActor* Actor : AllActors)
		{
			Layers->DisassociateActorFromLayers(Actor);
			World->EditorDestroyActor(Actor, true);
		}
	}

	//Select newly created actor
	GEditor->SelectNone(false, true, false);
	GEditor->SelectActor(OutMergedActor, true, false);
	GEditor->NoteSelectionChange();

	return true;
}

bool UStaticMeshEditorSubsystem::CreateProxyMeshActor(const TArray<class AStaticMeshActor*>& ActorsToMerge, const FCreateProxyMeshActorOptions& MergeOptions, class AStaticMeshActor*& OutMergedActor)
{
	// See FMeshProxyTool::RunMerge (Engine\Source\Editor\MergeActors\Private\MeshProxyTool\MeshProxyTool.cpp)
	TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, true);

	OutMergedActor = nullptr;

	if (!EditorScriptingHelpers::CheckIfInEditorAndPIE())
	{
		return false;
	}

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	if (!UnrealEditorSubsystem)
	{
		return false;
	}

	FString FailureReason;
	FString PackageName = EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(MergeOptions.BasePackageName, FailureReason);
	if (PackageName.IsEmpty())
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "CreateProxyMeshActor. Failed to convert the BasePackageName. %ls", *FailureReason);
		return false;
	}

	// Cleanup actors
	TArray<AStaticMeshActor*> StaticMeshActors;
	TArray<UPrimitiveComponent*> AllComponents_UNUSED;
	FVector PivotLocation;
	if (!InternalEditorMeshLibrary::FindValidActorAndComponents(ActorsToMerge, StaticMeshActors, AllComponents_UNUSED, PivotLocation, FailureReason))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "CreateProxyMeshActor failed. %ls", *FailureReason);
		return false;
	}
	TArray<AActor*> AllActors(StaticMeshActors);

	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	FCreateProxyDelegate ProxyDelegate;
	TArray<UObject*> CreatedAssets;
	ProxyDelegate.BindLambda([&CreatedAssets](const FGuid Guid, TArray<UObject*>& InAssetsToSync) {CreatedAssets.Append(InAssetsToSync); });

	MeshUtilities.CreateProxyMesh(
		AllActors,                      // List of Actors to merge
		MergeOptions.MeshProxySettings, // Merge settings
		nullptr,                        // Base Material used for final proxy material. Note: nullptr for default impl: GEngine->DefaultFlattenMaterial
		nullptr,                        // Package for generated assets. Note: if nullptr, BasePackageName is used
		PackageName,                    // Will be used for naming generated assets, in case InOuter is not specified ProxyBasePackageName will be used as long package name for creating new packages
		FGuid::NewGuid(),               // Identify a job, First argument of the ProxyDelegate
		ProxyDelegate                   // Called back on asset creation
	);

	UStaticMesh* MergedMesh = nullptr;
	if (!CreatedAssets.FindItemByClass(&MergedMesh))
	{
		UE_LOGF(LogStaticMeshEditorSubsystem, Error, "CreateProxyMeshActor failed. No mesh created.");
		return false;
	}

	// Update the asset registry that a new static mesh and material has been created
	FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	for (UObject* Asset : CreatedAssets)
	{
		AssetRegistry.AssetCreated(Asset);
		GEditor->BroadcastObjectReimported(Asset);
	}

	// Also notify the content browser that the new assets exists
	if (!IsRunningCommandlet())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(CreatedAssets, true);
	}

	// Place new mesh in the world
	UWorld* ActorWorld = AllActors[0]->GetWorld();
	ULevel* ActorLevel = AllActors[0]->GetLevel();
	if (MergeOptions.bSpawnMergedActor)
	{
		FActorSpawnParameters Params;
		Params.OverrideLevel = ActorLevel;
		OutMergedActor = ActorWorld->SpawnActor<AStaticMeshActor>(FVector::ZeroVector, FRotator::ZeroRotator, Params);
		if (!OutMergedActor)
		{
			UE_LOGF(LogStaticMeshEditorSubsystem, Error, "CreateProxyMeshActor failed. Internal error while creating the merged actor.");
			return false;
		}

		OutMergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		OutMergedActor->SetActorLabel(MergeOptions.NewActorLabel);
		ActorWorld->UpdateCullDistanceVolumes(OutMergedActor, OutMergedActor->GetStaticMeshComponent());
	}

	// Remove source actors
	if (MergeOptions.bDestroySourceActors)
	{
		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		for (AActor* Actor : AllActors)
		{
			Layers->DisassociateActorFromLayers(Actor);
			ActorWorld->EditorDestroyActor(Actor, true);
		}
	}

	//Select newly created actor
	if (OutMergedActor)
	{
		GEditor->SelectNone(false, true, false);
		GEditor->SelectActor(OutMergedActor, true, false); // don't notify but manually call NoteSelectionChange ?
		GEditor->NoteSelectionChange();
	}

	return true;
}


#undef LOCTEXT_NAMESPACE
