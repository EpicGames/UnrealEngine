// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorTools.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EngineDefines.h"
#include "Styling/AppStyle.h"
#include "PropertyHandle.h"
#include "IDetailGroup.h"
#include "IDetailChildrenBuilder.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "RawMesh.h"
#include "MeshUtilities.h"
#include "StaticMeshResources.h"
#include "StaticMeshEditor.h"
#include "PropertyCustomizationHelpers.h"
#include "MaterialList.h"
#include "PhysicsEngine/BodySetup.h"
#include "FbxMeshUtils.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "SPerQualityLevelPropertiesWidget.h"
#include "PlatformInfo.h"
#include "ContentStreaming.h"
#include "EditorDirectories.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/SkeletalMesh.h"
#include "EngineAnalytics.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IMeshReductionManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "JsonObjectConverter.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ScopedTransaction.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Input/STextComboBox.h"
#include "PerPlatformPropertyCustomization.h"
#include "Misc/ScopedSlowTask.h"
#include "MeshCardRepresentation.h"
#include "NaniteDefinitions.h"
#include "NaniteLayout.h"

const uint32 MaxHullCount = 64;
const uint32 MinHullCount = 1;
const uint32 DefaultHullCount = 4;
const uint32 HullCountDelta = 1;

const uint32 MaxHullPrecision = 1000000;
const uint32 MinHullPrecision = 10000;
const uint32 DefaultHullPrecision = 100000;
const uint32 HullPrecisionDelta = 10000;


const int32 MaxVertsPerHullCount = 32;
const int32 MinVertsPerHullCount = 6;
const int32 DefaultVertsPerHull = 16;

#define LOCTEXT_NAMESPACE "StaticMeshEditor"
DEFINE_LOG_CATEGORY_STATIC(LogStaticMeshEditorTools,Log,All);

bool GIsNaniteStaticMeshSettingsInitiallyCollapsed = 0;
static FAutoConsoleVariableRef CVarIsNaniteStaticMeshSettingsInitiallyCollapsed(
	TEXT("r.Nanite.IsNaniteStaticMeshSettingsInitiallyCollapsed"),
	GIsNaniteStaticMeshSettingsInitiallyCollapsed,
	TEXT("If the Nanite Settings are initially collapsed in the details panel in the Static Mesh Editor Tool."),
	ECVF_ReadOnly
);

static bool GEnableStaticMeshMaterialSlotOverlayMaterialUI = false;
static FAutoConsoleVariableRef CCvarEnableStaticMeshMaterialSlotOverlayMaterialUI(
	TEXT("StaticMesh.FeatureFlags.UI.ShowMaterialSlotOverlayMaterial"),
	GEnableStaticMeshMaterialSlotOverlayMaterialUI,
	TEXT("Whether Static mesh editor show material overlay per material slot UI."),
	ECVF_Default);

/**
 * Window for Nanite settings.
 */

typedef Nanite::FSettingsLayout<UStaticMesh, true /* ForceEnable */, true /* HighRes */> FNaniteStaticMeshLayoutImpl;

class FNaniteStaticMeshLayout : public TSharedFromThis<FNaniteStaticMeshLayout>
{
public:
	FNaniteStaticMeshLayout(FStaticMeshEditor& InStaticMeshEditor)
	: StaticMeshEditor(InStaticMeshEditor)
	{
		LayoutImpl = MakeShareable(new FNaniteStaticMeshLayoutImpl());

		LayoutImpl->OnGetMesh = TDelegate<UStaticMesh*()>::CreateLambda([this]
		{
			return StaticMeshEditor.GetStaticMesh();
		});

		LayoutImpl->OnRefreshTool = TDelegate<void()>::CreateLambda([this]
		{
			StaticMeshEditor.RefreshTool();
		});

		const UStaticMesh* StaticMesh = LayoutImpl->GetMesh();
		check(StaticMesh);
		LayoutImpl->UpdateSettings(StaticMesh->GetNaniteSettings());
	}

	~FNaniteStaticMeshLayout()
	{
	}

	void AddToDetailsPanel(IDetailLayoutBuilder& DetailBuilder)
	{
		const bool bInitiallyCollapsed = GIsNaniteStaticMeshSettingsInitiallyCollapsed != 0;

		UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
		TWeakObjectPtr<UStaticMesh> WeakStaticMesh = StaticMesh;
		const int32 SortOrder = 10;
		LayoutImpl->AddToDetailsPanel(WeakStaticMesh, DetailBuilder, SortOrder, bInitiallyCollapsed);
	}

	inline bool IsApplyNeeded() const
	{
		return LayoutImpl->IsApplyNeeded();
	}

	inline void ApplyChanges()
	{
		return LayoutImpl->ApplyChanges();
	}

private:
	// The Static Mesh Editor this tool is associated with.
	FStaticMeshEditor& StaticMeshEditor;

	TSharedPtr<FNaniteStaticMeshLayoutImpl> LayoutImpl;
};

/*
* Custom data key
*/
enum SM_CustomDataKey
{
	CustomDataKey_LODVisibilityState = 0, //This is the key to know if a LOD is shown in custom mode. Do CustomDataKey_LODVisibilityState + LodIndex for a specific LOD
	CustomDataKey_LODEditMode = 100 //This is the key to know the state of the custom lod edit mode.
};


FStaticMeshDetails::FStaticMeshDetails( class FStaticMeshEditor& InStaticMeshEditor )
	: StaticMeshEditor( InStaticMeshEditor )
{}

FStaticMeshDetails::~FStaticMeshDetails()
{
}

void FStaticMeshDetails::CustomizeDetails( class IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& LODSettingsCategory = DetailBuilder.EditCategory( "LodSettings", LOCTEXT("LodSettingsCategory", "LOD Settings") );
	IDetailCategoryBuilder& StaticMeshCategory = DetailBuilder.EditCategory( "StaticMesh", LOCTEXT("StaticMeshGeneralSettings", "General Settings") );
	IDetailCategoryBuilder& CollisionCategory = DetailBuilder.EditCategory( "Collision", LOCTEXT("CollisionCategory", "Collision") );
	IDetailCategoryBuilder& ImportSettingsCategory = DetailBuilder.EditCategory("ImportSettings");

	TSharedRef<IPropertyHandle> LightMapCoordinateIndexProperty = DetailBuilder.GetProperty(UStaticMesh::GetLightMapCoordinateIndexName());
	TSharedRef<IPropertyHandle> LightMapResolutionProperty = DetailBuilder.GetProperty(UStaticMesh::GetLightMapResolutionName());
	LightMapCoordinateIndexProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FStaticMeshDetails::OnLightmapSettingsChanged));
	LightMapResolutionProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FStaticMeshDetails::OnLightmapSettingsChanged));

	TSharedRef<IPropertyHandle> StaticMaterials = DetailBuilder.GetProperty(UStaticMesh::GetStaticMaterialsName());
	StaticMaterials->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> ImportSettings = DetailBuilder.GetProperty(UStaticMesh::GetAssetImportDataMemberName());
	if (!StaticMeshEditor.GetStaticMesh() || 
		!StaticMeshEditor.GetStaticMesh()->GetAssetImportData() ||
		!StaticMeshEditor.GetStaticMesh()->GetAssetImportData()->IsA<UFbxStaticMeshImportData>())
	{
		ImportSettings->MarkResetToDefaultCustomized();

		IDetailPropertyRow& Row = ImportSettingsCategory.AddProperty(ImportSettings);
		Row.CustomWidget(true)
			.NameContent()
			[
				ImportSettings->CreatePropertyNameWidget()
			];
	}
	else
	{
		// If the AssetImportData is an instance of UFbxStaticMeshImportData we create a custom UI.
		// Since DetailCustomization UI is not supported on instanced properties and because IDetailLayoutBuilder does not work well inside instanced objects scopes,
		// we need to manually recreate the whole FbxStaticMeshImportData UI in order to customize it.
		ImportSettings->MarkHiddenByCustomization();
		VertexColorImportOptionHandle = ImportSettings->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, VertexColorImportOption));
		VertexColorImportOverrideHandle = ImportSettings->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, VertexOverrideColor));
		TMap<FName, IDetailGroup*> ExistingGroup;
		PropertyCustomizationHelpers::MakeInstancedPropertyCustomUI(ExistingGroup, ImportSettingsCategory, ImportSettings, FOnInstancedPropertyIteration::CreateSP(this, &FStaticMeshDetails::OnInstancedFbxStaticMeshImportDataPropertyIteration));
	}

	DetailBuilder.EditCategory( "Navigation", FText::GetEmpty(), ECategoryPriority::Uncommon );

	LevelOfDetailSettings = MakeShareable( new FLevelOfDetailSettingsLayout( StaticMeshEditor ) );
	LevelOfDetailSettings->AddToDetailsPanel( DetailBuilder );

	// Hide the existing NaniteSettings property so we can use the customization instead
	TSharedRef<IPropertyHandle> NaniteSettingsProperty = DetailBuilder.GetProperty(UStaticMesh::GetNaniteSettingsMemberName());
	NaniteSettingsProperty->MarkHiddenByCustomization();

	NaniteSettings = MakeShareable(new FNaniteStaticMeshLayout(StaticMeshEditor));
	NaniteSettings->AddToDetailsPanel(DetailBuilder);

	TSharedPtr<IPropertyHandle> SupportRayTracingProperty = DetailBuilder.GetProperty(UStaticMesh::GetSupportRayTracingMemberName());

	// Hide the existing RayTracingProxySettings property so we can use the customization instead
	TSharedRef<IPropertyHandle> RayTracingProxySettingsProperty = DetailBuilder.GetProperty(UStaticMesh::GetRayTracingProxySettingsMemberName());
	RayTracingProxySettingsProperty->MarkHiddenByCustomization();

	RayTracingProxySettings = MakeShareable(new FRayTracingProxySettingsLayout(StaticMeshEditor));
	
	IDetailCategoryBuilder& RayTracingCategory = DetailBuilder.EditCategory("RayTracing");
	RayTracingCategory.AddProperty(SupportRayTracingProperty); // need to manually add bSupportRayTracing so it shows up first 
	RayTracingCategory.AddCustomBuilder(RayTracingProxySettings.ToSharedRef());

	TSharedRef<IPropertyHandle> BodyProp = DetailBuilder.GetProperty(UStaticMesh::GetBodySetupName());
	BodyProp->MarkHiddenByCustomization();

	static TArray<FName> HiddenBodyInstanceProps;

	if( HiddenBodyInstanceProps.Num() == 0 )
	{
		//HiddenBodyInstanceProps.Add("DefaultInstance");
		HiddenBodyInstanceProps.Add("BoneName");
		HiddenBodyInstanceProps.Add("PhysicsType");
		HiddenBodyInstanceProps.Add("bConsiderForBounds");
		HiddenBodyInstanceProps.Add("CollisionReponse");
	}

	uint32 NumChildren = 0;
	BodyProp->GetNumChildren( NumChildren );

	TSharedPtr<IPropertyHandle> BodyPropObject;

	if( NumChildren == 1 )
	{
		// This is an edit inline new property so the first child is the object instance for the edit inline new.  The instance contains the child we want to display
		BodyPropObject = BodyProp->GetChildHandle( 0 );

		NumChildren = 0;
		BodyPropObject->GetNumChildren( NumChildren );

		for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
		{
			TSharedPtr<IPropertyHandle> ChildProp = BodyPropObject->GetChildHandle(ChildIndex);
			if( ChildProp.IsValid() && ChildProp->GetProperty() && !HiddenBodyInstanceProps.Contains(ChildProp->GetProperty()->GetFName()) )
			{
				CollisionCategory.AddProperty( ChildProp );
			}
		}
	}
}

void FStaticMeshDetails::OnInstancedFbxStaticMeshImportDataPropertyIteration(IDetailCategoryBuilder& BaseCategory, IDetailGroup* PropertyGroup, TSharedRef<IPropertyHandle>& Property) const
{
	IDetailPropertyRow* Row = nullptr;

	if (Property->GetBoolMetaData(TEXT("ReimportRestrict")))
	{
		//Dont create a row for reimport restricted property, we do not want to show them
		return;
	}

	if (PropertyGroup)
	{
		Row = &PropertyGroup->AddPropertyRow(Property);
	}
	else
	{
		Row = &BaseCategory.AddProperty(Property);
	}

	if (Row)
	{
		//Vertex Override Color property should be disabled if we are not in override mode.
		if (Property->IsValidHandle() && Property->GetProperty() == VertexColorImportOverrideHandle->GetProperty())
		{
			Row->IsEnabled(TAttribute<bool>(this, &FStaticMeshDetails::GetVertexOverrideColorEnabledState));
		}
	}
}

void FStaticMeshDetails::OnLightmapSettingsChanged()
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->EnforceLightmapRestrictions(false);
}

bool FStaticMeshDetails::GetVertexOverrideColorEnabledState() const
{
	uint8 VertexColorImportOption;
	check(VertexColorImportOptionHandle.IsValid());
	ensure(VertexColorImportOptionHandle->GetValue(VertexColorImportOption) == FPropertyAccess::Success);

	return (VertexColorImportOption == EVertexColorImportOption::Override);
}

void SConvexDecomposition::Construct(const FArguments& InArgs)
{
	StaticMeshEditorPtr = InArgs._StaticMeshEditorPtr;
	CurrentHullPrecision = DefaultHullPrecision;
	CurrentHullCount = DefaultHullCount;
	CurrentMaxVertsPerHullCount = DefaultVertsPerHull;

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 8.0f, 0.0f, 8.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("HullCount_ConvexDecomp", "Hull Count"))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(3.0f)
			[	
				SAssignNew(HullCount, SSpinBox<uint32>)
				.ToolTipText(LOCTEXT("HullCount_ConvexDecomp_Tip", "Maximum number of convex pieces that will be created."))
				.MinValue(MinHullCount)
				.MaxValue(MaxHullCount)
				.Delta(HullCountDelta)
				.Value(this, &SConvexDecomposition::GetHullCount)
				.OnValueCommitted(this, &SConvexDecomposition::OnHullCountCommitted)
				.OnValueChanged(this, &SConvexDecomposition::OnHullCountChanged)
			]
		]


		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 8.0f, 0.0f, 8.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("MaxHullVerts_ConvexDecomp", "Max Hull Verts") )
			]

			+SHorizontalBox::Slot()
			.FillWidth(3.0f)
			[
				SAssignNew(MaxVertsPerHull, SSpinBox<int32>)
				.ToolTipText(LOCTEXT("MaxHullVerts_ConvexDecomp_Tip", "Maximum number of vertices allowed for any generated convex hull."))
				.MinValue(MinVertsPerHullCount)
				.MaxValue(MaxVertsPerHullCount)
				.Value( this, &SConvexDecomposition::GetVertsPerHullCount )
				.OnValueCommitted( this, &SConvexDecomposition::OnVertsPerHullCountCommitted )
				.OnValueChanged( this, &SConvexDecomposition::OnVertsPerHullCountChanged )
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 8.0f, 0.0f, 8.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("HullPrecision_ConvexDecomp", "Hull Precision"))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(3.0f)
			[
				SAssignNew(HullPrecision, SSpinBox<uint32>)
				.ToolTipText(LOCTEXT("HullPrecision_ConvexDecomp_Tip", "Number of voxels to use when generating collision."))
				.MinValue(MinHullPrecision)
				.MaxValue(MaxHullPrecision)
				.Delta(HullPrecisionDelta)
				.Value(this, &SConvexDecomposition::GetHullPrecision)
				.OnValueCommitted(this, &SConvexDecomposition::OnHullPrecisionCommitted)
				.OnValueChanged(this, &SConvexDecomposition::OnHullPrecisionChanged)
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SButton)
				.Text( LOCTEXT("Apply_ConvexDecomp", "Apply") )
				.OnClicked(this, &SConvexDecomposition::OnApplyDecomp)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 8.0f, 0.0f)
			[
				SNew(SButton)
				.Text( LOCTEXT("Defaults_ConvexDecomp", "Defaults") )
				.OnClicked(this, &SConvexDecomposition::OnDefaults)
			]
		]
	];
}

bool FStaticMeshDetails::IsApplyNeeded() const
{
	return
		(LevelOfDetailSettings.IsValid() && LevelOfDetailSettings->IsApplyNeeded()) ||
		(NaniteSettings.IsValid() && NaniteSettings->IsApplyNeeded());
}

void FStaticMeshDetails::ApplyChanges()
{
	if (LevelOfDetailSettings.IsValid())
	{
		LevelOfDetailSettings->ApplyChanges();
	}

	if (NaniteSettings.IsValid())
	{
		NaniteSettings->ApplyChanges();
	}
}

SConvexDecomposition::~SConvexDecomposition()
{

}

FReply SConvexDecomposition::OnApplyDecomp()
{
	StaticMeshEditorPtr.Pin()->DoDecomp(CurrentHullCount, CurrentMaxVertsPerHullCount, CurrentHullPrecision);

	return FReply::Handled();
}

FReply SConvexDecomposition::OnDefaults()
{
	CurrentHullCount = DefaultHullCount;
	CurrentHullPrecision = DefaultHullPrecision;
	CurrentMaxVertsPerHullCount = DefaultVertsPerHull;


	return FReply::Handled();
}

void SConvexDecomposition::OnHullCountCommitted(uint32 InNewValue, ETextCommit::Type CommitInfo)
{
	OnHullCountChanged(InNewValue);
}

void SConvexDecomposition::OnHullCountChanged(uint32 InNewValue)
{
	CurrentHullCount = InNewValue;
}

uint32 SConvexDecomposition::GetHullCount() const
{
	return CurrentHullCount;
}

void SConvexDecomposition::OnHullPrecisionCommitted(uint32 InNewValue, ETextCommit::Type CommitInfo)
{
	OnHullPrecisionChanged(InNewValue);
}

void SConvexDecomposition::OnHullPrecisionChanged(uint32 InNewValue)
{
	CurrentHullPrecision = InNewValue;
}

uint32 SConvexDecomposition::GetHullPrecision() const
{
	return CurrentHullPrecision;
}

void SConvexDecomposition::OnVertsPerHullCountCommitted(int32 InNewValue,  ETextCommit::Type CommitInfo)
{
	OnVertsPerHullCountChanged(InNewValue);
}

void SConvexDecomposition::OnVertsPerHullCountChanged(int32 InNewValue)
{
	CurrentMaxVertsPerHullCount = InNewValue;
}

int32 SConvexDecomposition::GetVertsPerHullCount() const
{
	return CurrentMaxVertsPerHullCount;
}

static UEnum& GetFeatureImportanceEnum()
{
	static UEnum* FeatureImportanceEnum = nullptr;
	if (FeatureImportanceEnum == nullptr)
	{
		FTopLevelAssetPath MeshFeatureImportanceEnumPath(TEXT("/Script/Engine"), TEXT("EMeshFeatureImportance"));
		FeatureImportanceEnum = FindObject<UEnum>(MeshFeatureImportanceEnumPath);
		check(FeatureImportanceEnum);
	}
	return *FeatureImportanceEnum;
}

static UEnum& GetTerminationCriterionEunum()
{
	static UEnum* EnumPtr = nullptr;
	if (EnumPtr == nullptr)
	{
		FTopLevelAssetPath StaticMeshReductionTerimationCriterionEnumPath(TEXT("/Script/Engine"), TEXT("EStaticMeshReductionTerimationCriterion"));
		EnumPtr = FindObject<UEnum>(StaticMeshReductionTerimationCriterionEnumPath);
		check(EnumPtr);
	}
	return *EnumPtr;
}

static void FillEnumOptions(TArray<TSharedPtr<FString> >& OutStrings, UEnum& InEnum)
{
	for (int32 EnumIndex = 0; EnumIndex < InEnum.NumEnums() - 1; ++EnumIndex)
	{
		OutStrings.Add(MakeShareable(new FString(InEnum.GetNameStringByIndex(EnumIndex))));
	}
}

FMeshBuildSettingsLayout::FMeshBuildSettingsLayout( TSharedRef<FLevelOfDetailSettingsLayout> InParentLODSettings, int32 InLODIndex)
	: ParentLODSettings( InParentLODSettings )
	, LODIndex(InLODIndex)
{

}

FMeshBuildSettingsLayout::~FMeshBuildSettingsLayout()
{
}

void FMeshBuildSettingsLayout::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	NodeRow.NameContent()
	[
		SNew( STextBlock )
		.Text( LOCTEXT("MeshBuildSettings", "Build Settings") )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];
}

FString FMeshBuildSettingsLayout::GetCurrentDistanceFieldReplacementMeshPath() const
{
	return BuildSettings.DistanceFieldReplacementMesh ? BuildSettings.DistanceFieldReplacementMesh->GetPathName() : FString("");
}

void FMeshBuildSettingsLayout::OnDistanceFieldReplacementMeshSelected(const FAssetData& AssetData)
{
	BuildSettings.DistanceFieldReplacementMesh = Cast<UStaticMesh>(AssetData.GetAsset());
}

void FMeshBuildSettingsLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RecomputeNormals", "Recompute Normals") )
		.RowTag("RecomputeNormals")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RecomputeNormals", "Recompute Normals"))
		
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldRecomputeNormals)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnRecomputeNormalsChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RecomputeTangents", "Recompute Tangents") )
		.RowTag("RecomputeTangents")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RecomputeTangents", "Recompute Tangents"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldRecomputeTangents)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnRecomputeTangentsChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseMikkTSpace", "Use MikkTSpace Tangent Space") )
		.RowTag("UseMikkTSpace")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseMikkTSpace", "Use MikkTSpace Tangent Space"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldUseMikkTSpace)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnUseMikkTSpaceChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ComputeWeightedNormals", "Compute Weighted Normals") )
		.RowTag("ComputeWeightedNormals")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("ComputeWeightedNormals", "Compute Weighted Normals"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldComputeWeightedNormals)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnComputeWeightedNormalsChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RemoveDegenerates", "Remove Degenerates") )
		.RowTag("RemoveDegenerates")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RemoveDegenerates", "Remove Degenerates"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsEnabled(this, &FMeshBuildSettingsLayout::IsRemoveDegeneratesDisabled)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldRemoveDegenerates)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnRemoveDegeneratesChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("BuildReversedIndexBuffer", "Build Reversed Index Buffer") )
		.RowTag("BuildReversedIndexBuffer")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("BuildReversedIndexBuffer", "Build Reversed Index Buffer"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldBuildReversedIndexBuffer)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnBuildReversedIndexBufferChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("UseHighPrecisionTangentBasis", "Use High Precision Tangent Basis"))
		.RowTag("UseHighPrecisionTangentBasis")
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("UseHighPrecisionTangentBasis", "Use High Precision Tangent Basis"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldUseHighPrecisionTangentBasis)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnUseHighPrecisionTangentBasisChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseFullPrecisionUVs", "Use Full Precision UVs") )
		.RowTag("UseFullPrecisionUVs")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseFullPrecisionUVs", "Use Full Precision UVs"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldUseFullPrecisionUVs)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnUseFullPrecisionUVsChanged)
		];
	}
	
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseBackwardsCompatibleF16TruncUVs", "UE4 Compatible UVs") )
		.RowTag("UseBackwardsCompatibleF16TruncUVs")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseBackwardsCompatibleF16TruncUVs", "UE4 Compatible UVs"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldUseBackwardsCompatibleF16TruncUVs)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnUseBackwardsCompatibleF16TruncUVsChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("GenerateLightmapUVs", "Generate Lightmap UVs") )
		.RowTag("GenerateLightmapUVs")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("GenerateLightmapUVs", "Generate Lightmap UVs"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldGenerateLightmapUVs)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnGenerateLightmapUVsChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("MinLightmapResolution", "Min Lightmap Resolution") )
		.RowTag("MinLightmapResolution")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("MinLightmapResolution", "Min Lightmap Resolution"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<int32>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(1)
			.MaxValue(2048)
			.Value(this, &FMeshBuildSettingsLayout::GetMinLightmapResolution)
			.OnValueChanged(this, &FMeshBuildSettingsLayout::OnMinLightmapResolutionChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("SourceLightmapIndex", "Source Lightmap Index") )
		.RowTag("SourceLightmapIndex")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("SourceLightmapIndex", "Source Lightmap Index"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<int32>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(0)
			.MaxValue(7)
			.Value(this, &FMeshBuildSettingsLayout::GetSrcLightmapIndex)
			.OnValueChanged(this, &FMeshBuildSettingsLayout::OnSrcLightmapIndexChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("DestinationLightmapIndex", "Destination Lightmap Index") )
		.RowTag("DestinationLightmapIndex")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("DestinationLightmapIndex", "Destination Lightmap Index"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<int32>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(0)
			.MaxValue(7)
			.Value(this, &FMeshBuildSettingsLayout::GetDstLightmapIndex)
			.OnValueChanged(this, &FMeshBuildSettingsLayout::OnDstLightmapIndexChanged)
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("BuildScale", "Build Scale"))
		.RowTag("BuildScale")
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("BuildScale", "Build Scale"))
			.ToolTipText( LOCTEXT("BuildScale_ToolTip", "The local scale applied when building the mesh") )
		]
		.ValueContent()
		.MinDesiredWidth(125.0f * 3.0f)
		.MaxDesiredWidth(125.0f * 3.0f)
		[
			SNew(SVectorInputBox)
			.X(this, &FMeshBuildSettingsLayout::GetBuildScaleX)
			.Y(this, &FMeshBuildSettingsLayout::GetBuildScaleY)
			.Z(this, &FMeshBuildSettingsLayout::GetBuildScaleZ)
			.bColorAxisLabels(false)
			.AllowSpin(false)
			.OnXCommitted(this, &FMeshBuildSettingsLayout::OnBuildScaleXChanged)
			.OnYCommitted(this, &FMeshBuildSettingsLayout::OnBuildScaleYChanged)
			.OnZCommitted(this, &FMeshBuildSettingsLayout::OnBuildScaleZChanged)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("DistanceFieldResolutionScale", "Distance Field Resolution Scale") )
		.RowTag("DistanceFieldResolutionScale")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("DistanceFieldResolutionScale", "Distance Field Resolution Scale"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<float>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(0.0f)
			.MaxValue(100.0f)
			.Value(this, &FMeshBuildSettingsLayout::GetDistanceFieldResolutionScale)
			.OnValueChanged(this, &FMeshBuildSettingsLayout::OnDistanceFieldResolutionScaleChanged)
			.OnValueCommitted(this, &FMeshBuildSettingsLayout::OnDistanceFieldResolutionScaleCommitted)
		];
	}
		
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("GenerateDistanceFieldAsIfTwoSided", "Two-Sided Distance Field Generation") )
		.RowTag("GenerateDistanceFieldAsIfTwoSided")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("GenerateDistanceFieldAsIfTwoSided", "Two-Sided Distance Field Generation"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshBuildSettingsLayout::ShouldGenerateDistanceFieldAsIfTwoSided)
			.OnCheckStateChanged(this, &FMeshBuildSettingsLayout::OnGenerateDistanceFieldAsIfTwoSidedChanged)
		];
	}

	{
		TSharedRef<SWidget> PropWidget = SNew(SObjectPropertyEntryBox)
			.AllowedClass(UStaticMesh::StaticClass())
			.AllowClear(true)
			.ObjectPath(this, &FMeshBuildSettingsLayout::GetCurrentDistanceFieldReplacementMeshPath)
			.OnObjectChanged(this, &FMeshBuildSettingsLayout::OnDistanceFieldReplacementMeshSelected);

		ChildrenBuilder.AddCustomRow( LOCTEXT("DistanceFieldReplacementMesh", "Distance Field Replacement Mesh") )
		.RowTag("DistanceFieldReplacementMesh")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("DistanceFieldReplacementMesh", "Distance Field Replacement Mesh"))
		]
		.ValueContent()
		[
			PropWidget
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MaxLumenMeshCards", "Max Lumen Mesh Cards"))
		.RowTag("MaxLumenMeshCards")
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("MaxLumenMeshCards", "Max Lumen Mesh Cards"))
			]
		.ValueContent()
			[
				SNew(SSpinBox<int32>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0)
			.MaxValue(32)
			.Value(this, &FMeshBuildSettingsLayout::GetMaxLumenMeshCards)
			.OnValueChanged(this, &FMeshBuildSettingsLayout::OnMaxLumenMeshCardsChanged)
			.OnValueCommitted(this, &FMeshBuildSettingsLayout::OnMaxLumenMeshCardsCommitted)
			];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ApplyChanges", "Apply Changes") )
		.RowTag("ApplyChanges")
		.ValueContent()
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.OnClicked(this, &FMeshBuildSettingsLayout::OnApplyChanges)
			.IsEnabled(ParentLODSettings.Pin().ToSharedRef(), &FLevelOfDetailSettingsLayout::IsApplyNeeded)
			[
				SNew( STextBlock )
				.Text(LOCTEXT("ApplyChanges", "Apply Changes"))
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	}
}

void FMeshBuildSettingsLayout::UpdateSettings(const FMeshBuildSettings& InSettings)
{
	BuildSettings = InSettings;
}

FReply FMeshBuildSettingsLayout::OnApplyChanges()
{
	if( ParentLODSettings.IsValid() )
	{
		ParentLODSettings.Pin()->ApplyChanges();
	}
	return FReply::Handled();
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldRecomputeNormals() const
{
	return BuildSettings.bRecomputeNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldRecomputeTangents() const
{
	return BuildSettings.bRecomputeTangents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldUseMikkTSpace() const
{
	return BuildSettings.bUseMikkTSpace ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldComputeWeightedNormals() const
{
	return BuildSettings.bComputeWeightedNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldRemoveDegenerates() const
{
	return BuildSettings.bRemoveDegenerates ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldBuildReversedIndexBuffer() const
{
	return BuildSettings.bBuildReversedIndexBuffer ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldUseHighPrecisionTangentBasis() const
{
	return BuildSettings.bUseHighPrecisionTangentBasis ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldUseFullPrecisionUVs() const
{
	return BuildSettings.bUseFullPrecisionUVs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldUseBackwardsCompatibleF16TruncUVs() const
{
	return BuildSettings.bUseBackwardsCompatibleF16TruncUVs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldGenerateLightmapUVs() const
{
	return BuildSettings.bGenerateLightmapUVs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FMeshBuildSettingsLayout::ShouldGenerateDistanceFieldAsIfTwoSided() const
{
	return BuildSettings.bGenerateDistanceFieldAsIfTwoSided ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FMeshBuildSettingsLayout::IsRemoveDegeneratesDisabled() const
{
	if (TSharedPtr<FLevelOfDetailSettingsLayout> LODSettingsLayout = ParentLODSettings.Pin())
	{
		return !LODSettingsLayout->IsNaniteEnabled();
	}

	return true;
}

int32 FMeshBuildSettingsLayout::GetMinLightmapResolution() const
{
	return BuildSettings.MinLightmapResolution;
}

int32 FMeshBuildSettingsLayout::GetSrcLightmapIndex() const
{
	return BuildSettings.SrcLightmapIndex;
}

int32 FMeshBuildSettingsLayout::GetDstLightmapIndex() const
{
	return BuildSettings.DstLightmapIndex;
}

TOptional<float> FMeshBuildSettingsLayout::GetBuildScaleX() const
{
	return static_cast<float>(BuildSettings.BuildScale3D.X);
}

TOptional<float> FMeshBuildSettingsLayout::GetBuildScaleY() const
{
	return static_cast<float>(BuildSettings.BuildScale3D.Y);
}

TOptional<float> FMeshBuildSettingsLayout::GetBuildScaleZ() const
{
	return static_cast<float>(BuildSettings.BuildScale3D.Z);
}

float FMeshBuildSettingsLayout::GetDistanceFieldResolutionScale() const
{
	return BuildSettings.DistanceFieldResolutionScale;
}

int32 FMeshBuildSettingsLayout::GetMaxLumenMeshCards() const
{
	return BuildSettings.MaxLumenMeshCards;
}

void FMeshBuildSettingsLayout::OnRecomputeNormalsChanged(ECheckBoxState NewState)
{
	const bool bRecomputeNormals = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRecomputeNormals != bRecomputeNormals)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bRecomputeNormals"), bRecomputeNormals ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bRecomputeNormals = bRecomputeNormals;
	}
}

void FMeshBuildSettingsLayout::OnRecomputeTangentsChanged(ECheckBoxState NewState)
{
	const bool bRecomputeTangents = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRecomputeTangents != bRecomputeTangents)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bRecomputeTangents"), bRecomputeTangents ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bRecomputeTangents = bRecomputeTangents;
	}
}

void FMeshBuildSettingsLayout::OnUseMikkTSpaceChanged(ECheckBoxState NewState)
{
	const bool bUseMikkTSpace = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseMikkTSpace != bUseMikkTSpace)
	{
		BuildSettings.bUseMikkTSpace = bUseMikkTSpace;
	}
}

void FMeshBuildSettingsLayout::OnComputeWeightedNormalsChanged(ECheckBoxState NewState)
{
	const bool bComputeWeightedNormals = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bComputeWeightedNormals != bComputeWeightedNormals)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bComputeWeightedNormals"), bComputeWeightedNormals ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bComputeWeightedNormals = bComputeWeightedNormals;
	}
}

void FMeshBuildSettingsLayout::OnRemoveDegeneratesChanged(ECheckBoxState NewState)
{
	const bool bRemoveDegenerates = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRemoveDegenerates != bRemoveDegenerates)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bRemoveDegenerates"), bRemoveDegenerates ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bRemoveDegenerates = bRemoveDegenerates;
	}
}

void FMeshBuildSettingsLayout::OnBuildReversedIndexBufferChanged(ECheckBoxState NewState)
{
	const bool bBuildReversedIndexBuffer = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bBuildReversedIndexBuffer != bBuildReversedIndexBuffer)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bBuildReversedIndexBuffer"), bBuildReversedIndexBuffer ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bBuildReversedIndexBuffer = bBuildReversedIndexBuffer;
	}
}

void FMeshBuildSettingsLayout::OnUseHighPrecisionTangentBasisChanged(ECheckBoxState NewState)
{
	const bool bUseHighPrecisionTangents = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseHighPrecisionTangentBasis != bUseHighPrecisionTangents)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bUseHighPrecisionTangentBasis"), bUseHighPrecisionTangents ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bUseHighPrecisionTangentBasis = bUseHighPrecisionTangents;
	}
}

void FMeshBuildSettingsLayout::OnUseFullPrecisionUVsChanged(ECheckBoxState NewState)
{
	const bool bUseFullPrecisionUVs = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseFullPrecisionUVs != bUseFullPrecisionUVs)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bUseFullPrecisionUVs"), bUseFullPrecisionUVs ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bUseFullPrecisionUVs = bUseFullPrecisionUVs;
	}
}

void FMeshBuildSettingsLayout::OnUseBackwardsCompatibleF16TruncUVsChanged(ECheckBoxState NewState)
{
	const bool bUseBackwardsCompatibleF16TruncUVs = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseBackwardsCompatibleF16TruncUVs != bUseBackwardsCompatibleF16TruncUVs)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bUseBackwardsCompatibleF16TruncUVs"), bUseBackwardsCompatibleF16TruncUVs ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bUseBackwardsCompatibleF16TruncUVs = bUseBackwardsCompatibleF16TruncUVs;
	}
}

void FMeshBuildSettingsLayout::OnGenerateLightmapUVsChanged(ECheckBoxState NewState)
{
	const bool bGenerateLightmapUVs = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bGenerateLightmapUVs != bGenerateLightmapUVs)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bGenerateLightmapUVs"), bGenerateLightmapUVs ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bGenerateLightmapUVs = bGenerateLightmapUVs;
	}
}

void FMeshBuildSettingsLayout::OnGenerateDistanceFieldAsIfTwoSidedChanged(ECheckBoxState NewState)
{
	const bool bGenerateDistanceFieldAsIfTwoSided = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bGenerateDistanceFieldAsIfTwoSided != bGenerateDistanceFieldAsIfTwoSided)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("bGenerateDistanceFieldAsIfTwoSided"), bGenerateDistanceFieldAsIfTwoSided ? TEXT("True") : TEXT("False"));
		}
		BuildSettings.bGenerateDistanceFieldAsIfTwoSided = bGenerateDistanceFieldAsIfTwoSided;
	}
}

void FMeshBuildSettingsLayout::OnMinLightmapResolutionChanged( int32 NewValue )
{
	if (BuildSettings.MinLightmapResolution != NewValue)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("MinLightmapResolution"), FString::Printf(TEXT("%i"), NewValue));
		}
		BuildSettings.MinLightmapResolution = NewValue;
	}
}

void FMeshBuildSettingsLayout::OnSrcLightmapIndexChanged( int32 NewValue )
{
	if (BuildSettings.SrcLightmapIndex != NewValue)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("SrcLightmapIndex"), FString::Printf(TEXT("%i"), NewValue));
		}
		BuildSettings.SrcLightmapIndex = NewValue;
	}
}

void FMeshBuildSettingsLayout::OnDstLightmapIndexChanged( int32 NewValue )
{
	if (BuildSettings.DstLightmapIndex != NewValue)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("DstLightmapIndex"), FString::Printf(TEXT("%i"), NewValue));
		}
		BuildSettings.DstLightmapIndex = NewValue;
	}
}

void FMeshBuildSettingsLayout::OnBuildScaleXChanged( float NewScaleX, ETextCommit::Type TextCommitType )
{
	if (!FMath::IsNearlyEqual(NewScaleX, 0.0f) && BuildSettings.BuildScale3D.X != NewScaleX)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("BuildScale3D.X"), FString::Printf(TEXT("%.3f"), NewScaleX));
		}
		BuildSettings.BuildScale3D.X = NewScaleX;
	}
}

void FMeshBuildSettingsLayout::OnBuildScaleYChanged( float NewScaleY, ETextCommit::Type TextCommitType )
{
	if (!FMath::IsNearlyEqual(NewScaleY, 0.0f) && BuildSettings.BuildScale3D.Y != NewScaleY)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("BuildScale3D.Y"), FString::Printf(TEXT("%.3f"), NewScaleY));
		}
		BuildSettings.BuildScale3D.Y = NewScaleY;
	}
}

void FMeshBuildSettingsLayout::OnBuildScaleZChanged( float NewScaleZ, ETextCommit::Type TextCommitType )
{
	if (!FMath::IsNearlyEqual(NewScaleZ, 0.0f) && BuildSettings.BuildScale3D.Z != NewScaleZ)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("BuildScale3D.Z"), FString::Printf(TEXT("%.3f"), NewScaleZ));
		}
		BuildSettings.BuildScale3D.Z = NewScaleZ;
	}
}

void FMeshBuildSettingsLayout::OnDistanceFieldResolutionScaleChanged(float NewValue)
{
	BuildSettings.DistanceFieldResolutionScale = NewValue;
}

void FMeshBuildSettingsLayout::OnDistanceFieldResolutionScaleCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("DistanceFieldResolutionScale"), FString::Printf(TEXT("%.3f"), NewValue));
	}
	OnDistanceFieldResolutionScaleChanged(NewValue);
}

void FMeshBuildSettingsLayout::OnMaxLumenMeshCardsChanged(int32 NewValue)
{
	BuildSettings.MaxLumenMeshCards = NewValue;
}

void FMeshBuildSettingsLayout::OnMaxLumenMeshCardsCommitted(int32 NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.BuildSettings"), TEXT("MaxLumenMeshCards"), FString::Printf(TEXT("%d"), NewValue));
	}
	OnMaxLumenMeshCardsChanged(NewValue);
}


FMeshReductionSettingsLayout::FMeshReductionSettingsLayout( TSharedRef<FLevelOfDetailSettingsLayout> InParentLODSettings, int32 InCurrentLODIndex, bool InCanReduceMyself)
	: ParentLODSettings( InParentLODSettings )
	, CurrentLODIndex(InCurrentLODIndex)
	, bCanReduceMyself(InCanReduceMyself)
{

	FillEnumOptions(ImportanceOptions, GetFeatureImportanceEnum());

	FillEnumOptions(TerminationOptions, GetTerminationCriterionEunum());

	bUseQuadricSimplifier = UseNativeToolLayout();
}

FMeshReductionSettingsLayout::~FMeshReductionSettingsLayout()
{
}

void FMeshReductionSettingsLayout::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow  )
{
	NodeRow.NameContent()
	[
		SNew( STextBlock )
		.Text( LOCTEXT("MeshReductionSettings", "Reduction Settings") )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];
}

bool FMeshReductionSettingsLayout::UseNativeToolLayout() const 
{
	// Are we using our tool, or simplygon?  The tool is only changed during editor restarts
	IMeshReduction* ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetStaticMeshReductionInterface();

	FString VersionString = ReductionModule->GetVersionString();
	TArray<FString> SplitVersionString;
	VersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);

	bool bUseQuadricSimplier = SplitVersionString[0].Equals("QuadricMeshReduction");
	return bUseQuadricSimplier;
}

EVisibility FMeshReductionSettingsLayout::GetTriangleCriterionVisibility() const
{
	EVisibility VisibilityValue;
	if (!bUseQuadricSimplifier
		|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Triangles
		|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Any)
	{
		VisibilityValue =  EVisibility::Visible;
	}
	else
	{
		VisibilityValue = EVisibility::Hidden;
	}
	return VisibilityValue;

}


EVisibility FMeshReductionSettingsLayout::GetVertexCriterionVisibility() const
{
	EVisibility VisibilityValue;
	if (!bUseQuadricSimplifier
		|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Vertices
		|| ReductionSettings.TerminationCriterion == EStaticMeshReductionTerimationCriterion::Any)
	{
		VisibilityValue = EVisibility::Visible;
	}
	else
	{
		VisibilityValue = EVisibility::Hidden;
	}
	return VisibilityValue;

}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMeshReductionSettingsLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{

	if (bUseQuadricSimplifier)
	{

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("Termination_MeshSimplification", "Termination"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Termination_MeshSimplification", "Termination"))
				]
			.ValueContent()
			[
				SAssignNew(TerminationCriterionCombo, STextComboBox)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.OptionsSource(&TerminationOptions)
				.InitiallySelectedItem(TerminationOptions[static_cast<int32>(ReductionSettings.TerminationCriterion)])
				.OnSelectionChanged(this, &FMeshReductionSettingsLayout::OnTerminationCriterionChanged)
			];

		}
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("PercentTriangles", "Percent Triangles") )
		.RowTag("PercentTriangles")
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("PercentTriangles", "Percent Triangles"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<float>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(0.0f)
			.MaxValue(100.0f)
			.Value(this, &FMeshReductionSettingsLayout::GetPercentTriangles)
			.OnValueChanged(this, &FMeshReductionSettingsLayout::OnPercentTrianglesChanged)
			.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnPercentTrianglesCommitted)
		]
		.Visibility(TAttribute<EVisibility>(this, &FMeshReductionSettingsLayout::GetTriangleCriterionVisibility));
		
		if (bUseQuadricSimplifier)
		{
			ChildrenBuilder.AddCustomRow( LOCTEXT("MaxTrianglesPercentage_Row", "Max Number of Triangles") )
			.RowTag("MaxPercentTriangles")
			.NameContent()
			[
				SNew(STextBlock)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.Text(LOCTEXT("MaxTrianglesPercentage", "Max Triangles Count"))
				.ToolTipText(LOCTEXT("MaxTrianglesPercentage_ToolTip", "The maximum number of triangles to retain when using percentage criterion."))
			]
			.ValueContent()
			[
				SNew(SSpinBox<uint32>)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.MinValue(2)
				.MaxValue(MAX_uint32)
				.Value(this, &FMeshReductionSettingsLayout::GetMaxNumOfPercentTriangles)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnMaxNumOfPercentTrianglesChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnMaxNumOfPercentTrianglesCommitted)
			]
			.Visibility(TAttribute<EVisibility>(this, &FMeshReductionSettingsLayout::GetTriangleCriterionVisibility));
		}
	}

	if (bUseQuadricSimplifier)
	{
		//Percent vertices
		ChildrenBuilder.AddCustomRow(LOCTEXT("PercentVertices", "Percent Vertices"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("PercentVertices", "Percent Vertices"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<float>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0.0f)
				.MaxValue(100.0f)
				.Value(this, &FMeshReductionSettingsLayout::GetPercentVertices)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnPercentVerticesChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnPercentVerticesCommitted)
			]
		.Visibility(TAttribute<EVisibility>(this, &FMeshReductionSettingsLayout::GetVertexCriterionVisibility));

		//Max percent absolute vertices
		ChildrenBuilder.AddCustomRow( LOCTEXT("MaxVerticesPercentage_Row", "Max Number of Vertices") )
			.RowTag("MaxVertices")
			.NameContent()
			[
				SNew(STextBlock)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.Text(LOCTEXT("MaxVerticesPercentage", "Max Vertices Count"))
				.ToolTipText(LOCTEXT("MaxVerticesPercentage_ToolTip", "The maximum number of Vertices to retain when using percentage criterion."))
			]
			.ValueContent()
			[
				SNew(SSpinBox<uint32>)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.MinValue(2)
				.MaxValue(MAX_uint32)
				.Value(this, &FMeshReductionSettingsLayout::GetMaxNumOfPercentVertices)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnMaxNumOfPercentVerticesChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnMaxNumOfPercentVerticesCommitted)
			]
			.Visibility(TAttribute<EVisibility>(this, &FMeshReductionSettingsLayout::GetVertexCriterionVisibility));

	}

	// Controls that only simplygon uses.
	if (!bUseQuadricSimplifier)
	{
		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("MaxDeviation", "Max Deviation"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaxDeviation", "Max Deviation"))
				]
			.ValueContent()
				[
					SNew(SSpinBox<float>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0.0f)
				.MaxValue(1000.0f)
				.Value(this, &FMeshReductionSettingsLayout::GetMaxDeviation)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnMaxDeviationChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnMaxDeviationCommitted)
				];

		}

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("PixelError", "Pixel Error"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("PixelError", "Pixel Error"))
				]
			.ValueContent()
				[
					SNew(SSpinBox<float>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0.0f)
				.MaxValue(40.0f)
				.Value(this, &FMeshReductionSettingsLayout::GetPixelError)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnPixelErrorChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnPixelErrorCommitted)
				];

		}

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("Silhouette_MeshSimplification", "Silhouette"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Silhouette_MeshSimplification", "Silhouette"))
				]
			.ValueContent()
				[
					SAssignNew(SilhouetteCombo, STextComboBox)
					//.Font( IDetailLayoutBuilder::GetDetailFont() )
				.ContentPadding(0.f)
				.OptionsSource(&ImportanceOptions)
				.InitiallySelectedItem(ImportanceOptions[ReductionSettings.SilhouetteImportance])
				.OnSelectionChanged(this, &FMeshReductionSettingsLayout::OnSilhouetteImportanceChanged)
				];

		}

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("Texture_MeshSimplification", "Texture"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Texture_MeshSimplification", "Texture"))
				]
			.ValueContent()
				[
					SAssignNew(TextureCombo, STextComboBox)
					//.Font( IDetailLayoutBuilder::GetDetailFont() )
				.ContentPadding(0.f)
				.OptionsSource(&ImportanceOptions)
				.InitiallySelectedItem(ImportanceOptions[ReductionSettings.TextureImportance])
				.OnSelectionChanged(this, &FMeshReductionSettingsLayout::OnTextureImportanceChanged)
				];

		}

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("Shading_MeshSimplification", "Shading"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Shading_MeshSimplification", "Shading"))
				]
			.ValueContent()
				[
					SAssignNew(ShadingCombo, STextComboBox)
					//.Font( IDetailLayoutBuilder::GetDetailFont() )
				.ContentPadding(0.f)
				.OptionsSource(&ImportanceOptions)
				.InitiallySelectedItem(ImportanceOptions[ReductionSettings.ShadingImportance])
				.OnSelectionChanged(this, &FMeshReductionSettingsLayout::OnShadingImportanceChanged)
				];

		}
	}
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("WeldingThreshold", "Welding Threshold") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("WeldingThreshold", "Welding Threshold"))
		]
		.ValueContent()
		[
			SNew(SSpinBox<float>)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.MinValue(0.0f)
			.MaxValue(10.0f)
			.Value(this, &FMeshReductionSettingsLayout::GetWeldingThreshold)
			.OnValueChanged(this, &FMeshReductionSettingsLayout::OnWeldingThresholdChanged)
			.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnWeldingThresholdCommitted)
		];

	}

	// controls that only simplygon uses
	if (!bUseQuadricSimplifier)
	{
		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("RecomputeNormals", "Recompute Normals"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("RecomputeNormals", "Recompute Normals"))

				]
			.ValueContent()
				[
					SNew(SCheckBox)
					.IsChecked(this, &FMeshReductionSettingsLayout::ShouldRecalculateNormals)
				.OnCheckStateChanged(this, &FMeshReductionSettingsLayout::OnRecalculateNormalsChanged)
				];
		}

		{
			ChildrenBuilder.AddCustomRow(LOCTEXT("HardEdgeAngle", "Hard Edge Angle"))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("HardEdgeAngle", "Hard Edge Angle"))
				]
			.ValueContent()
				[
					SNew(SSpinBox<float>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0.0f)
				.MaxValue(180.0f)
				.Value(this, &FMeshReductionSettingsLayout::GetHardAngleThreshold)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::OnHardAngleThresholdChanged)
				.OnValueCommitted(this, &FMeshReductionSettingsLayout::OnHardAngleThresholdCommitted)
				];

		}
	}

	//Base LOD
	{
		int32 MaxBaseReduceIndex = bCanReduceMyself ? CurrentLODIndex : CurrentLODIndex - 1;
		ChildrenBuilder.AddCustomRow(LOCTEXT("ReductionBaseLOD", "Base LOD"))
			.NameContent()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ReductionBaseLOD", "Base LOD"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(HAlign_Left)
			[
				SNew(SNumericEntryBox<int32>)
				.AllowSpin(true)
				.MinSliderValue(0)
				.MaxSliderValue(MaxBaseReduceIndex)
				.MinValue(0)
				.MaxValue(MaxBaseReduceIndex)
				.Value(this, &FMeshReductionSettingsLayout::GetBaseLODIndex)
				.OnValueChanged(this, &FMeshReductionSettingsLayout::SetBaseLODIndex)
			];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ApplyChanges", "Apply Changes") )
			.RowTag("ApplyChanges")
			.ValueContent()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.OnClicked(this, &FMeshReductionSettingsLayout::OnApplyChanges)
				.IsEnabled(ParentLODSettings.Pin().ToSharedRef(), &FLevelOfDetailSettingsLayout::IsApplyNeeded)
				[
					SNew( STextBlock )
					.Text(LOCTEXT("ApplyChanges", "Apply Changes"))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
			];
	}

	if (!bUseQuadricSimplifier)
	{
		SilhouetteCombo->SetSelectedItem(ImportanceOptions[ReductionSettings.SilhouetteImportance]);
		TextureCombo->SetSelectedItem(ImportanceOptions[ReductionSettings.TextureImportance]);
		ShadingCombo->SetSelectedItem(ImportanceOptions[ReductionSettings.ShadingImportance]);
	}
	else
	{
		TerminationCriterionCombo->SetSelectedItem(TerminationOptions[static_cast<int32>(ReductionSettings.TerminationCriterion)]);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

const FMeshReductionSettings& FMeshReductionSettingsLayout::GetSettings() const
{
	return ReductionSettings;
}

void FMeshReductionSettingsLayout::UpdateSettings(const FMeshReductionSettings& InSettings)
{
	ReductionSettings = InSettings;
}

FReply FMeshReductionSettingsLayout::OnApplyChanges()
{
	if( ParentLODSettings.IsValid() )
	{
		ParentLODSettings.Pin()->ApplyChanges();
	}
	return FReply::Handled();
}

float FMeshReductionSettingsLayout::GetPercentTriangles() const
{
	return ReductionSettings.PercentTriangles * 100.0f; // Display fraction as percentage.
}

uint32 FMeshReductionSettingsLayout::GetMaxNumOfPercentTriangles() const
{
	return ReductionSettings.MaxNumOfTriangles;
}

float FMeshReductionSettingsLayout::GetPercentVertices() const
{
	return ReductionSettings.PercentVertices * 100.0f; // Display fraction as percentage.
}

uint32 FMeshReductionSettingsLayout::GetMaxNumOfPercentVertices() const
{
	return ReductionSettings.MaxNumOfVerts;
}

float FMeshReductionSettingsLayout::GetMaxDeviation() const
{
	return ReductionSettings.MaxDeviation;
}

float FMeshReductionSettingsLayout::GetPixelError() const
{
	return ReductionSettings.PixelError;
}

float FMeshReductionSettingsLayout::GetWeldingThreshold() const
{
	return ReductionSettings.WeldingThreshold;
}

ECheckBoxState FMeshReductionSettingsLayout::ShouldRecalculateNormals() const
{
	return ReductionSettings.bRecalculateNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

float FMeshReductionSettingsLayout::GetHardAngleThreshold() const
{
	return ReductionSettings.HardAngleThreshold;
}

void FMeshReductionSettingsLayout::OnPercentTrianglesChanged(float NewValue)
{
	// Percentage -> fraction.
	ReductionSettings.PercentTriangles = NewValue * 0.01f;
}

void FMeshReductionSettingsLayout::OnPercentVerticesChanged(float NewValue)
{
	// Percentage -> fraction.
	ReductionSettings.PercentVertices = NewValue * 0.01f;
}

void FMeshReductionSettingsLayout::OnPercentTrianglesCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("PercentTriangles"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnPercentTrianglesChanged(NewValue);
}


void FMeshReductionSettingsLayout::OnPercentVerticesCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("PercentVertices"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnPercentVerticesChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnMaxNumOfPercentTrianglesChanged(uint32 NewValue)
{
	ReductionSettings.MaxNumOfTriangles = NewValue;
}

void FMeshReductionSettingsLayout::OnMaxNumOfPercentTrianglesCommitted(uint32 NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("MaxNumOfTrianglesPercentage"), FString::Printf(TEXT("%d"), NewValue));
	}
	OnMaxNumOfPercentTrianglesChanged(NewValue);
}


void FMeshReductionSettingsLayout::OnMaxNumOfPercentVerticesChanged(uint32 NewValue)
{
	ReductionSettings.MaxNumOfVerts = NewValue;
}

void FMeshReductionSettingsLayout::OnMaxNumOfPercentVerticesCommitted(uint32 NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("MaxNumOfVertsPercentage"), FString::Printf(TEXT("%d"), NewValue));
	}
	OnMaxNumOfPercentVerticesChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnMaxDeviationChanged(float NewValue)
{
	ReductionSettings.MaxDeviation = NewValue;
}

void FMeshReductionSettingsLayout::OnMaxDeviationCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("MaxDeviation"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnMaxDeviationChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnPixelErrorChanged(float NewValue)
{
	ReductionSettings.PixelError = NewValue;
}

void FMeshReductionSettingsLayout::OnPixelErrorCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("PixelError"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnPixelErrorChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnWeldingThresholdChanged(float NewValue)
{
	ReductionSettings.WeldingThreshold = NewValue;
}

void FMeshReductionSettingsLayout::OnWeldingThresholdCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("WeldingThreshold"), FString::Printf(TEXT("%.2f"), NewValue));
	}
	OnWeldingThresholdChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnRecalculateNormalsChanged(ECheckBoxState NewValue)
{
	const bool bRecalculateNormals = NewValue == ECheckBoxState::Checked;
	if (ReductionSettings.bRecalculateNormals != bRecalculateNormals)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("bRecalculateNormals"), bRecalculateNormals ? TEXT("True") : TEXT("False"));
		}
		ReductionSettings.bRecalculateNormals = bRecalculateNormals;
	}
}

void FMeshReductionSettingsLayout::OnHardAngleThresholdChanged(float NewValue)
{
	ReductionSettings.HardAngleThreshold = NewValue;
}

void FMeshReductionSettingsLayout::OnHardAngleThresholdCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("HardAngleThreshold"), FString::Printf(TEXT("%.3f"), NewValue));
	}
	OnHardAngleThresholdChanged(NewValue);
}

void FMeshReductionSettingsLayout::OnSilhouetteImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	const EMeshFeatureImportance::Type SilhouetteImportance = (EMeshFeatureImportance::Type)ImportanceOptions.Find(NewValue);
	if (ReductionSettings.SilhouetteImportance != SilhouetteImportance)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("SilhouetteImportance"), *NewValue.Get());
		}
		ReductionSettings.SilhouetteImportance = SilhouetteImportance;
	}
}

void FMeshReductionSettingsLayout::OnTextureImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	const EMeshFeatureImportance::Type TextureImportance = (EMeshFeatureImportance::Type)ImportanceOptions.Find(NewValue);
	if (ReductionSettings.TextureImportance != TextureImportance)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("TextureImportance"), *NewValue.Get());
		}
		ReductionSettings.TextureImportance = TextureImportance;
	}
}

void FMeshReductionSettingsLayout::OnShadingImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	const EMeshFeatureImportance::Type ShadingImportance = (EMeshFeatureImportance::Type)ImportanceOptions.Find(NewValue);
	if (ReductionSettings.ShadingImportance != ShadingImportance)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("ShadingImportance"), *NewValue.Get());
		}
		ReductionSettings.ShadingImportance = ShadingImportance;
	}
}

void FMeshReductionSettingsLayout::OnTerminationCriterionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	const EStaticMeshReductionTerimationCriterion TerminationCriterion = (EStaticMeshReductionTerimationCriterion)TerminationOptions.Find(NewValue);
	if (ReductionSettings.TerminationCriterion != TerminationCriterion)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("TerminationCriterion"), *NewValue.Get());
		}
		ReductionSettings.TerminationCriterion = TerminationCriterion;
	}
}

TOptional<int32> FMeshReductionSettingsLayout::GetBaseLODIndex() const
{
	return ReductionSettings.BaseLODModel;
}

void FMeshReductionSettingsLayout::SetBaseLODIndex(int32 NewLODBaseIndex)
{
	if (NewLODBaseIndex <= CurrentLODIndex)
	{
		ReductionSettings.BaseLODModel = NewLODBaseIndex;
	}
}

FMeshSectionSettingsLayout::~FMeshSectionSettingsLayout()
{
}

UStaticMesh& FMeshSectionSettingsLayout::GetStaticMesh() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	return *StaticMesh;
}

void FMeshSectionSettingsLayout::AddToCategory( IDetailCategoryBuilder& CategoryBuilder )
{
	FSectionListDelegates SectionListDelegates;

	SectionListDelegates.OnGetSections.BindSP(this, &FMeshSectionSettingsLayout::OnGetSectionsForView, LODIndex);
	SectionListDelegates.OnSectionChanged.BindSP(this, &FMeshSectionSettingsLayout::OnSectionChanged);
	SectionListDelegates.OnGenerateCustomNameWidgets.BindSP(this, &FMeshSectionSettingsLayout::OnGenerateCustomNameWidgetsForSection);
	SectionListDelegates.OnGenerateCustomSectionWidgets.BindSP(this, &FMeshSectionSettingsLayout::OnGenerateCustomSectionWidgetsForSection);

	SectionListDelegates.OnCopySectionList.BindSP(this, &FMeshSectionSettingsLayout::OnCopySectionList, LODIndex);
	SectionListDelegates.OnCanCopySectionList.BindSP(this, &FMeshSectionSettingsLayout::OnCanCopySectionList, LODIndex);
	SectionListDelegates.OnPasteSectionList.BindSP(this, &FMeshSectionSettingsLayout::OnPasteSectionList, LODIndex);
	SectionListDelegates.OnCopySectionItem.BindSP(this, &FMeshSectionSettingsLayout::OnCopySectionItem);
	SectionListDelegates.OnCanCopySectionItem.BindSP(this, &FMeshSectionSettingsLayout::OnCanCopySectionItem);
	SectionListDelegates.OnPasteSectionItem.BindSP(this, &FMeshSectionSettingsLayout::OnPasteSectionItem);
	//We need a valid name if we want the section expand state to be saved
	FName StaticMeshSectionListName = FName(*(FString(TEXT("StaticMeshSectionListNameLOD_")) + FString::FromInt(LODIndex)));
	CategoryBuilder.AddCustomBuilder(MakeShareable(new FSectionList(CategoryBuilder.GetParentLayout(), SectionListDelegates, true, 48, LODIndex, StaticMeshSectionListName)));

	StaticMeshEditor.RegisterOnSelectedLODChanged(FOnSelectedLODChanged::CreateSP(this, &FMeshSectionSettingsLayout::UpdateLODCategoryVisibility), false);
}

void FMeshSectionSettingsLayout::OnCopySectionList(int32 CurrentLODIndex)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	UStaticMesh& StaticMesh = GetStaticMesh();
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

	if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
	{
		FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

		for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
		{
			const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

			TSharedPtr<FJsonObject> JSonSection = MakeShareable(new FJsonObject);

			JSonSection->SetNumberField(TEXT("MaterialIndex"), Section.MaterialIndex);
			JSonSection->SetBoolField(TEXT("EnableCollision"), Section.bEnableCollision);
			JSonSection->SetBoolField(TEXT("CastShadow"), Section.bCastShadow);

			RootJsonObject->SetObjectField(FString::Printf(TEXT("Section_%d"), SectionIndex), JSonSection);
		}
	}

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FMeshSectionSettingsLayout::OnCanCopySectionList(int32 CurrentLODIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

	if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
	{
		FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

		return LOD.Sections.Num() > 0;
	}

	return false;
}

void FMeshSectionSettingsLayout::OnPasteSectionList(int32 CurrentLODIndex)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	bool bResult = FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		UStaticMesh& StaticMesh = GetStaticMesh();
		FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

		if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
		{
			FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

			GetStaticMesh().PreEditChange(Property);

			FScopedTransaction Transaction(LOCTEXT("StaticMeshToolChangedPasteSectionList", "Staticmesh editor: Pasted section list"));
			GetStaticMesh().Modify();

			FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

			for (int32 SectionIndex = 0; SectionIndex < LOD.Sections.Num(); ++SectionIndex)
			{
				FStaticMeshSection& Section = LOD.Sections[SectionIndex];

				const TSharedPtr<FJsonObject>* JSonSection = nullptr;

				if (RootJsonObject->TryGetObjectField(FString::Printf(TEXT("Section_%d"), SectionIndex), JSonSection))
				{
					(*JSonSection)->TryGetNumberField(TEXT("MaterialIndex"), Section.MaterialIndex);
					(*JSonSection)->TryGetBoolField(TEXT("EnableCollision"), Section.bEnableCollision);
					(*JSonSection)->TryGetBoolField(TEXT("CastShadow"), Section.bCastShadow);

					// Update the section info
					FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
					Info.MaterialIndex = Section.MaterialIndex;
					Info.bCastShadow = Section.bCastShadow;
					Info.bEnableCollision = Section.bEnableCollision;

					StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
				}
			}

			CallPostEditChange(Property);
		}
	}
}

void FMeshSectionSettingsLayout::OnCopySectionItem(int32 CurrentLODIndex, int32 SectionIndex)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	UStaticMesh& StaticMesh = GetStaticMesh();
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

	if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
	{
		FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

		if (LOD.Sections.IsValidIndex(SectionIndex))
		{
			const FStaticMeshSection& Section = LOD.Sections[SectionIndex];

			RootJsonObject->SetNumberField(TEXT("MaterialIndex"), Section.MaterialIndex);
			RootJsonObject->SetBoolField(TEXT("EnableCollision"), Section.bEnableCollision);
			RootJsonObject->SetBoolField(TEXT("CastShadow"), Section.bCastShadow);
		}
	}

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FMeshSectionSettingsLayout::OnCanCopySectionItem(int32 CurrentLODIndex, int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

	if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
	{
		FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

		return LOD.Sections.IsValidIndex(SectionIndex);
	}

	return false;
}

void FMeshSectionSettingsLayout::OnPasteSectionItem(int32 CurrentLODIndex, int32 SectionIndex)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	bool bResult = FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		UStaticMesh& StaticMesh = GetStaticMesh();
		FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();

		if (RenderData != nullptr && RenderData->LODResources.IsValidIndex(CurrentLODIndex))
		{
			FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

			GetStaticMesh().PreEditChange(Property);

			FScopedTransaction Transaction(LOCTEXT("StaticMeshToolChangedPasteSectionItem", "Staticmesh editor: Pasted section item"));
			GetStaticMesh().Modify();

			FStaticMeshLODResources& LOD = RenderData->LODResources[CurrentLODIndex];

			if (LOD.Sections.IsValidIndex(SectionIndex))
			{
				FStaticMeshSection& Section = LOD.Sections[SectionIndex];

				RootJsonObject->TryGetNumberField(TEXT("MaterialIndex"), Section.MaterialIndex);
				RootJsonObject->TryGetBoolField(TEXT("EnableCollision"), Section.bEnableCollision);
				RootJsonObject->TryGetBoolField(TEXT("CastShadow"), Section.bCastShadow);

				// Update the section info
				FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
				Info.MaterialIndex = Section.MaterialIndex;
				Info.bCastShadow = Section.bCastShadow;
				Info.bEnableCollision = Section.bEnableCollision;

				StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
			}

			CallPostEditChange(Property);
		}
	}
}

void FMeshSectionSettingsLayout::OnGetSectionsForView(ISectionListBuilder& OutSections, int32 ForLODIndex)
{
	check(LODIndex == ForLODIndex);
	UStaticMesh& StaticMesh = GetStaticMesh();
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();
	if (RenderData && RenderData->LODResources.IsValidIndex(LODIndex))
	{
		FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
		int32 NumSections = LOD.Sections.Num();
		
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
			int32 MaterialIndex = Info.MaterialIndex;
			if (StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex))
			{
				FName CurrentSectionMaterialSlotName = StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialSlotName;
				FName CurrentSectionOriginalImportedMaterialName = StaticMesh.GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
				TMap<int32, FName> AvailableSectionName;
				int32 CurrentIterMaterialIndex = 0;
				for (const FStaticMaterial &SkeletalMaterial : StaticMesh.GetStaticMaterials())
				{
					if (MaterialIndex != CurrentIterMaterialIndex)
						AvailableSectionName.Add(CurrentIterMaterialIndex, SkeletalMaterial.MaterialSlotName);
					CurrentIterMaterialIndex++;
				}
				UMaterialInterface* SectionMaterial = StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialInterface;
				if (SectionMaterial == NULL)
				{
					SectionMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
				}
				//TODO: Need to know if a section material slot assignment was change from the default value (implemented in skeletalmesh editor)
				OutSections.AddSection(LODIndex, SectionIndex, CurrentSectionMaterialSlotName, MaterialIndex, CurrentSectionOriginalImportedMaterialName, AvailableSectionName, StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialInterface, false, false, MaterialIndex);
			}
		}
	}
}

void FMeshSectionSettingsLayout::OnSectionChanged(int32 ForLODIndex, int32 SectionIndex, int32 NewMaterialSlotIndex, FName NewMaterialSlotName)
{
	check(LODIndex == ForLODIndex);
	UStaticMesh& StaticMesh = GetStaticMesh();

	check(StaticMesh.GetStaticMaterials().IsValidIndex(NewMaterialSlotIndex));

	int32 NewStaticMaterialIndex = INDEX_NONE;
	for (int StaticMaterialIndex = 0; StaticMaterialIndex < StaticMesh.GetStaticMaterials().Num(); ++StaticMaterialIndex)
	{
		if (NewMaterialSlotIndex == StaticMaterialIndex && StaticMesh.GetStaticMaterials()[StaticMaterialIndex].MaterialSlotName == NewMaterialSlotName)
		{
			NewStaticMaterialIndex = StaticMaterialIndex;
			break;
		}
	}
	check(NewStaticMaterialIndex != INDEX_NONE);
	check(StaticMesh.GetRenderData());
	FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();
	if (RenderData && RenderData->LODResources.IsValidIndex(LODIndex))
	{
		bool bRefreshAll = false;
		FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
		if (LOD.Sections.IsValidIndex(SectionIndex))
		{
			FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

			GetStaticMesh().PreEditChange(Property);

			FScopedTransaction Transaction(LOCTEXT("StaticMeshOnSectionChangedTransaction", "Staticmesh editor: Section material slot changed"));
			GetStaticMesh().Modify();
			FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
			int32 CancelOldValue = Info.MaterialIndex;
			Info.MaterialIndex = NewStaticMaterialIndex;
			StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);

			CallPostEditChange();
		}
		if (bRefreshAll)
		{
			StaticMeshEditor.RefreshTool();
		}
	}
}

TSharedRef<SWidget> FMeshSectionSettingsLayout::OnGenerateCustomNameWidgetsForSection(int32 ForLODIndex, int32 SectionIndex)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshSectionSettingsLayout::IsSectionHighlighted, SectionIndex)
			.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionHighlightedChanged, SectionIndex)
			.ToolTipText(LOCTEXT("Highlight_ToolTip", "Highlights this section in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity( FLinearColor( 0.4f, 0.4f, 0.4f, 1.0f) )
				.Text(LOCTEXT("Highlight", "Highlight"))

			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 2, 0, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshSectionSettingsLayout::IsSectionIsolatedEnabled, SectionIndex)
			.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionIsolatedChanged, SectionIndex)
			.ToolTipText(LOCTEXT("Isolate_ToolTip", "Isolates this section in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
				.Text(LOCTEXT("Isolate", "Isolate"))

			]
		];
}

TSharedRef<SWidget> FMeshSectionSettingsLayout::OnGenerateCustomSectionWidgetsForSection(int32 ForLODIndex, int32 SectionIndex)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 2, 0)
		[
			SNew(SCheckBox)
				.IsChecked(this, &FMeshSectionSettingsLayout::DoesSectionCastShadow, SectionIndex)
				.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionCastShadowChanged, SectionIndex)
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("CastShadow", "Cast Shadow"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2,0,2,0)
		[
			SNew(SCheckBox)
				.IsEnabled(this, &FMeshSectionSettingsLayout::SectionCollisionEnabled)
				.ToolTipText(this, &FMeshSectionSettingsLayout::GetCollisionEnabledToolTip)
				.IsChecked(this, &FMeshSectionSettingsLayout::DoesSectionCollide, SectionIndex)
				.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionCollisionChanged, SectionIndex)
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("EnableCollision", "Enable Collision"))
			]
		]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 2, 0)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FMeshSectionSettingsLayout::IsSectionVisibleInRayTracing, SectionIndex)
			.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionVisibleInRayTracingChanged, SectionIndex)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("VisibleInRayTracing", "Visible In Ray Tracing"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 2, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshSectionSettingsLayout::DoesSectionAffectDistanceFieldLighting, SectionIndex)
			.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionAffectDistanceFieldLightingChanged, SectionIndex)
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("AffectDistanceFieldLighting", "Affect Distance Field Lighting"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 2, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshSectionSettingsLayout::IsSectionOpaque, SectionIndex)
			.OnCheckStateChanged(this, &FMeshSectionSettingsLayout::OnSectionForceOpaqueFlagChanged, SectionIndex)
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("ForceOpaque", "Force Opaque"))
			]
		];
}

ECheckBoxState FMeshSectionSettingsLayout::IsSectionVisibleInRayTracing(int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	return Info.bVisibleInRayTracing ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMeshSectionSettingsLayout::OnSectionVisibleInRayTracingChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FText TransactionTest = LOCTEXT("StaticMeshEditorSetVisibleInRayTracingSectionFlag", "Staticmesh editor: Set VisibleInRayTracing For section, the section will be visible in ray tracing effects");
	if (NewState == ECheckBoxState::Unchecked)
	{
		TransactionTest = LOCTEXT("StaticMeshEditorClearVisibleInRayTracingSectionFlag", "Staticmesh editor: Clear VisibleInRayTracing For section");
	}
	FScopedTransaction Transaction(TransactionTest);

	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

	StaticMesh.PreEditChange(Property);
	StaticMesh.Modify();

	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	Info.bVisibleInRayTracing = (NewState == ECheckBoxState::Checked) ? true : false;
	StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
	CallPostEditChange();
}


ECheckBoxState FMeshSectionSettingsLayout::DoesSectionAffectDistanceFieldLighting(int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	return Info.bAffectDistanceFieldLighting ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMeshSectionSettingsLayout::OnSectionAffectDistanceFieldLightingChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FText TransactionTest = LOCTEXT("StaticMeshEditorSetAffectDistanceFieldLightingSectionFlag", "Staticmesh editor: Set AffectDistanceFieldLighting For section, the section will be affect distance field lighting");
	if (NewState == ECheckBoxState::Unchecked)
	{
		TransactionTest = LOCTEXT("StaticMeshEditorClearAffectDistanceFieldLightingSectionFlag", "Staticmesh editor: Clear AffectDistanceFieldLighting For section");
	}
	FScopedTransaction Transaction(TransactionTest);

	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

	StaticMesh.PreEditChange(Property);
	StaticMesh.Modify();

	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	Info.bAffectDistanceFieldLighting = (NewState == ECheckBoxState::Checked) ? true : false;
	StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
	CallPostEditChange();
}


ECheckBoxState FMeshSectionSettingsLayout::IsSectionOpaque(int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	return Info.bForceOpaque ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMeshSectionSettingsLayout::OnSectionForceOpaqueFlagChanged( ECheckBoxState NewState, int32 SectionIndex )
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FText TransactionTest = LOCTEXT("StaticMeshEditorSetForceOpaqueSectionFlag", "Staticmesh editor: Set Force Opaque For section, the section will be considered opaque in ray tracing effects");
	if (NewState == ECheckBoxState::Unchecked)
	{
		TransactionTest = LOCTEXT("StaticMeshEditorClearForceOpaqueSectionFlag", "Staticmesh editor: Clear Force Opaque For section");
	}
	FScopedTransaction Transaction(TransactionTest);

	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

	StaticMesh.PreEditChange(Property);
	StaticMesh.Modify();

	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	Info.bForceOpaque = (NewState == ECheckBoxState::Checked) ? true : false;
	StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
	CallPostEditChange();
}

ECheckBoxState FMeshSectionSettingsLayout::DoesSectionCastShadow(int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	return Info.bCastShadow ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMeshSectionSettingsLayout::OnSectionCastShadowChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FText TransactionTest = LOCTEXT("StaticMeshEditorSetShadowCastingSectionFlag", "Staticmesh editor: Set Shadow Casting For section");
	if (NewState == ECheckBoxState::Unchecked)
	{
		TransactionTest = LOCTEXT("StaticMeshEditorClearShadowCastingSectionFlag", "Staticmesh editor: Clear Shadow Casting For section");
	}
	FScopedTransaction Transaction(TransactionTest);

	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

	StaticMesh.PreEditChange(Property);
	StaticMesh.Modify();
	
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	Info.bCastShadow = (NewState == ECheckBoxState::Checked) ? true : false;
	StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
	CallPostEditChange();
}

bool FMeshSectionSettingsLayout::SectionCollisionEnabled() const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	// Only enable 'Enable Collision' check box if this LOD is used for collision
	return (StaticMesh.LODForCollision == LODIndex);
}

FText FMeshSectionSettingsLayout::GetCollisionEnabledToolTip() const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	
	// If using a different LOD for collision, disable the check box
	if (StaticMesh.LODForCollision != LODIndex)
	{
		return LOCTEXT("EnableCollisionToolTipDisabled", "This LOD is not used for collision, see the LODForCollision setting.");
	}
	// This LOD is used for collision, give info on what flag does
	else
	{
		return LOCTEXT("EnableCollisionToolTipEnabled", "Controls whether this section ever has per-poly collision. Disabling this where possible will lower memory usage for this mesh.");
	}
}

ECheckBoxState FMeshSectionSettingsLayout::DoesSectionCollide(int32 SectionIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	return Info.bEnableCollision ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMeshSectionSettingsLayout::OnSectionCollisionChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FText TransactionTest = LOCTEXT("StaticMeshEditorSetCollisionSectionFlag", "Staticmesh editor: Set Collision For section");
	if (NewState == ECheckBoxState::Unchecked)
	{
		TransactionTest = LOCTEXT("StaticMeshEditorClearCollisionSectionFlag", "Staticmesh editor: Clear Collision For section");
	}
	FScopedTransaction Transaction(TransactionTest);

	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetSectionInfoMapName());

	StaticMesh.PreEditChange(Property);
	StaticMesh.Modify();

	FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
	Info.bEnableCollision = (NewState == ECheckBoxState::Checked) ? true : false;
	StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
	CallPostEditChange();
}

ECheckBoxState FMeshSectionSettingsLayout::IsSectionHighlighted(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		State = Component->SelectedEditorSection == SectionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FMeshSectionSettingsLayout::OnSectionHighlightedChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			Component->SelectedEditorSection = SectionIndex;
			if (Component->SectionIndexPreview != SectionIndex)
			{
				// Unhide all mesh sections
				Component->SetSectionPreview(INDEX_NONE);
			}
			Component->SetMaterialPreview(INDEX_NONE);
			Component->SelectedEditorMaterial = INDEX_NONE;
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			Component->SelectedEditorSection = INDEX_NONE;
		}
		Component->MarkRenderStateDirty();
		StaticMeshEditor.RefreshViewport();
	}
}

ECheckBoxState FMeshSectionSettingsLayout::IsSectionIsolatedEnabled(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		State = Component->SectionIndexPreview == SectionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FMeshSectionSettingsLayout::OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			Component->SetSectionPreview(SectionIndex);
			if (Component->SelectedEditorSection != SectionIndex)
			{
				Component->SelectedEditorSection = INDEX_NONE;
			}
			Component->SetMaterialPreview(INDEX_NONE);
			Component->SelectedEditorMaterial = INDEX_NONE;
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			Component->SetSectionPreview(INDEX_NONE);
		}
		Component->MarkRenderStateDirty();
		StaticMeshEditor.RefreshViewport();
	}
}

void FMeshSectionSettingsLayout::CallPostEditChange(FProperty* PropertyChanged/*=nullptr*/)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if( PropertyChanged )
	{
		FPropertyChangedEvent PropertyUpdateStruct(PropertyChanged);
		StaticMesh.PostEditChangeProperty(PropertyUpdateStruct);
	}
	else
	{
		StaticMesh.Modify();
		StaticMesh.PostEditChange();
	}
	if(StaticMesh.GetBodySetup())
	{
		StaticMesh.GetBodySetup()->CreatePhysicsMeshes();
	}
	StaticMeshEditor.RefreshViewport();
}

void FMeshSectionSettingsLayout::SetCurrentLOD(int32 NewLodIndex)
{
	if (StaticMeshEditor.GetStaticMeshComponent() == nullptr || LodCategoriesPtr == nullptr)
	{
		return;
	}
	int32 CurrentDisplayLOD = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
	int32 RealCurrentDisplayLOD = CurrentDisplayLOD == 0 ? 0 : CurrentDisplayLOD - 1;
	int32 RealNewLOD = NewLodIndex == 0 ? 0 : NewLodIndex - 1;
	
	if (CurrentDisplayLOD == NewLodIndex || !LodCategoriesPtr->IsValidIndex(RealCurrentDisplayLOD) || !LodCategoriesPtr->IsValidIndex(RealNewLOD))
	{
		return;
	}

	StaticMeshEditor.GetStaticMeshComponent()->SetForcedLodModel(NewLodIndex);

	//Reset the preview section since we do not edit the same LOD
	StaticMeshEditor.GetStaticMeshComponent()->SetSectionPreview(INDEX_NONE);
	StaticMeshEditor.GetStaticMeshComponent()->SelectedEditorSection = INDEX_NONE;
}

void FMeshSectionSettingsLayout::UpdateLODCategoryVisibility()
{
	if (StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		//Do not change the Category visibility if we are in custom mode
		return;
	}
	bool bAutoLod = false;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		bAutoLod = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel == 0;
	}
	int32 CurrentDisplayLOD = bAutoLod ? 0 : StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel - 1;

	if (LodCategoriesPtr != nullptr && LodCategoriesPtr->IsValidIndex(CurrentDisplayLOD) && StaticMeshEditor.GetStaticMesh())
	{
		int32 StaticMeshLodNumber = StaticMeshEditor.GetStaticMesh()->GetNumLODs();
		for (int32 LodCategoryIndex = 0; LodCategoryIndex < StaticMeshLodNumber; ++LodCategoryIndex)
		{
			if (!LodCategoriesPtr->IsValidIndex(LodCategoryIndex))
			{
				break;
			}
			(*LodCategoriesPtr)[LodCategoryIndex]->SetCategoryVisibility(CurrentDisplayLOD == LodCategoryIndex);
		}
		//Reset the preview section since we do not edit the same LOD
		StaticMeshEditor.GetStaticMeshComponent()->SetSectionPreview(INDEX_NONE);
		StaticMeshEditor.GetStaticMeshComponent()->SelectedEditorSection = INDEX_NONE;
	}
}

//////////////////////////////////////////////////////////////////////////
// FMeshMaterialLayout
//////////////////////////////////////////////////////////////////////////

FMeshMaterialsLayout::~FMeshMaterialsLayout()
{
}

UStaticMesh& FMeshMaterialsLayout::GetStaticMesh() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	return *StaticMesh;
}

void FMeshMaterialsLayout::AddToCategory(IDetailCategoryBuilder& CategoryBuilder, const TArray<FAssetData>& AssetDataArray)
{
	CategoryBuilder.AddCustomRow(LOCTEXT("AddLODLevelCategories_MaterialArrayOperationAdd", "Add Material Slot"))
		.RowTag("MaterialSlots")
		.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMeshMaterialsLayout::OnCopyMaterialList), FCanExecuteAction::CreateSP(this, &FMeshMaterialsLayout::OnCanCopyMaterialList)))
		.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMeshMaterialsLayout::OnPasteMaterialList)))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOperations", "Material Slots"))
		]
		.ValueContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FMeshMaterialsLayout::GetMaterialArrayText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 1.0f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd", "Add Material Slot"))
					.ToolTipText(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd_Tooltip", "Add Material Slot at the end of the Material slot array. Those Material slots can be used to override a LODs section, (not the base LOD)"))
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked(this, &FMeshMaterialsLayout::AddMaterialSlot)
					.IsEnabled(true)
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		];

	FMaterialListDelegates MaterialListDelegates;
	MaterialListDelegates.OnGetMaterials.BindSP(this, &FMeshMaterialsLayout::GetMaterials);
	MaterialListDelegates.OnMaterialChanged.BindSP(this, &FMeshMaterialsLayout::OnMaterialChanged);
	MaterialListDelegates.OnGenerateCustomMaterialWidgets.BindSP(this, &FMeshMaterialsLayout::OnGenerateWidgetsForMaterial);
	MaterialListDelegates.OnGenerateCustomNameWidgets.BindSP(this, &FMeshMaterialsLayout::OnGenerateNameWidgetsForMaterial);
	MaterialListDelegates.OnGenerateExtraBottomWidget.BindSP(this, &FMeshMaterialsLayout::OnGenerateMaterialListExtraBottomWidget);
	MaterialListDelegates.OnMaterialListDirty.BindSP(this, &FMeshMaterialsLayout::OnMaterialListDirty);
	MaterialListDelegates.OnResetMaterialToDefaultClicked.BindSP(this, &FMeshMaterialsLayout::OnResetMaterialToDefaultClicked);

	MaterialListDelegates.OnCopyMaterialItem.BindSP(this, &FMeshMaterialsLayout::OnCopyMaterialItem);
	MaterialListDelegates.OnCanCopyMaterialItem.BindSP(this, &FMeshMaterialsLayout::OnCanCopyMaterialItem);
	MaterialListDelegates.OnPasteMaterialItem.BindSP(this, &FMeshMaterialsLayout::OnPasteMaterialItem);

	CategoryBuilder.AddCustomBuilder(MakeShareable(new FMaterialList(CategoryBuilder.GetParentLayout(), MaterialListDelegates, AssetDataArray, false, true)));
}

void FMeshMaterialsLayout::OnCopyMaterialList()
{
	FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetStaticMaterialsName());
	check(Property != nullptr);

	auto JsonValue = FJsonObjectConverter::UPropertyToJsonValue(Property, &GetStaticMesh().GetStaticMaterials(), 0, 0);

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(JsonValue.ToSharedRef(), TEXT(""), Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FMeshMaterialsLayout::OnCanCopyMaterialList() const
{
	return GetStaticMesh().GetStaticMaterials().Num() > 0;
}

void FMeshMaterialsLayout::OnPasteMaterialList()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonValue> RootJsonValue;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonValue);

	if (RootJsonValue.IsValid())
	{
		FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetStaticMaterialsName());
		check(Property != nullptr);

		GetStaticMesh().PreEditChange(Property);
		FScopedTransaction Transaction(LOCTEXT("StaticMeshToolChangedPasteMaterialList", "Staticmesh editor: Pasted material list"));
		GetStaticMesh().Modify();

		TArray<FStaticMaterial> TempMaterials;
		FJsonObjectConverter::JsonValueToUProperty(RootJsonValue, Property, &TempMaterials, 0, 0);
		//Do not change the number of material in the array
		for (int32 MaterialIndex = 0; MaterialIndex < TempMaterials.Num(); ++MaterialIndex)
		{
			if (GetStaticMesh().GetStaticMaterials().IsValidIndex(MaterialIndex))
			{
				GetStaticMesh().GetStaticMaterials()[MaterialIndex].MaterialInterface = TempMaterials[MaterialIndex].MaterialInterface;
				GetStaticMesh().GetStaticMaterials()[MaterialIndex].OverlayMaterialInterface = TempMaterials[MaterialIndex].OverlayMaterialInterface;
			}
		}

		CallPostEditChange(Property);
	}
}

void FMeshMaterialsLayout::OnCopyMaterialItem(int32 CurrentSlot)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	if (GetStaticMesh().GetStaticMaterials().IsValidIndex(CurrentSlot))
	{
		const FStaticMaterial &Material = GetStaticMesh().GetStaticMaterials()[CurrentSlot];

		FJsonObjectConverter::UStructToJsonObject(FStaticMaterial::StaticStruct(), &Material, RootJsonObject, 0, 0);
	}

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FMeshMaterialsLayout::OnCanCopyMaterialItem(int32 CurrentSlot) const
{
	return GetStaticMesh().GetStaticMaterials().IsValidIndex(CurrentSlot);
}

void FMeshMaterialsLayout::OnPasteMaterialItem(int32 CurrentSlot)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		FProperty* Property = UStaticMesh::StaticClass()->FindPropertyByName(UStaticMesh::GetStaticMaterialsName());
		check(Property != nullptr);

		GetStaticMesh().PreEditChange(Property);

		FScopedTransaction Transaction(LOCTEXT("StaticMeshToolChangedPasteMaterialItem", "Staticmesh editor: Pasted material item"));
		GetStaticMesh().Modify();

		if (GetStaticMesh().GetStaticMaterials().IsValidIndex(CurrentSlot))
		{
			FStaticMaterial TmpStaticMaterial;
			FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FStaticMaterial::StaticStruct(), &TmpStaticMaterial, 0, 0);
			GetStaticMesh().GetStaticMaterials()[CurrentSlot].MaterialInterface = TmpStaticMaterial.MaterialInterface;
			GetStaticMesh().GetStaticMaterials()[CurrentSlot].OverlayMaterialInterface = TmpStaticMaterial.OverlayMaterialInterface;
		}

		CallPostEditChange(Property);
	}
}

FReply FMeshMaterialsLayout::AddMaterialSlot()
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FScopedTransaction Transaction(LOCTEXT("FMeshMaterialsLayout_AddMaterialSlot", "Staticmesh editor: Add material slot"));
	StaticMesh.Modify();
	StaticMesh.GetStaticMaterials().Add(FStaticMaterial());

	StaticMesh.PostEditChange();

	return FReply::Handled();
}

FText FMeshMaterialsLayout::GetMaterialArrayText() const
{
	UStaticMesh& StaticMesh = GetStaticMesh();

	FString MaterialArrayText = TEXT(" Material Slots");
	int32 SlotNumber = 0;
	SlotNumber = StaticMesh.GetStaticMaterials().Num();
	MaterialArrayText = FString::FromInt(SlotNumber) + MaterialArrayText;
	return FText::FromString(MaterialArrayText);
}

void FMeshMaterialsLayout::GetMaterials(IMaterialListBuilder& ListBuilder)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh.GetStaticMaterials().Num(); ++MaterialIndex)
	{
		UMaterialInterface* Material = StaticMesh.GetMaterial(MaterialIndex);
		if (Material == NULL)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		ListBuilder.AddMaterial(MaterialIndex, Material, /*bCanBeReplaced=*/ true);
	}
}

void FMeshMaterialsLayout::OnMaterialChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 MaterialIndex, bool bReplaceAll)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	StaticMesh.SetMaterial(MaterialIndex, NewMaterial);
	StaticMeshEditor.RefreshTool();
}

TSharedRef<SWidget> FMeshMaterialsLayout::OnGenerateWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	bool bMaterialIsUsed = false;
	if(MaterialUsedMap.Contains(SlotIndex))
	{
		bMaterialIsUsed = MaterialUsedMap.Find(SlotIndex)->Num() > 0;
	}

	return 
		SNew(SMaterialSlotWidget, SlotIndex, bMaterialIsUsed)
		.MaterialName(this, &FMeshMaterialsLayout::GetMaterialNameText, SlotIndex)
		.OnMaterialNameCommitted(this, &FMeshMaterialsLayout::OnMaterialNameCommitted, SlotIndex)
		.CanDeleteMaterialSlot(this, &FMeshMaterialsLayout::CanDeleteMaterialSlot, SlotIndex)
		.OnDeleteMaterialSlot(this, &FMeshMaterialsLayout::OnDeleteMaterialSlot, SlotIndex)
		.ToolTipText(this, &FMeshMaterialsLayout::GetOriginalImportMaterialNameText, SlotIndex);

#if 0 // HACK!!! Temporary disabled
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2,0,0)
		[
			SNew(SCheckBox)
				.Visibility(this, &FMeshMaterialsLayout::GetOverrideUVDensityVisibililty)
				.IsChecked(this, &FMeshMaterialsLayout::IsUVDensityOverridden, SlotIndex)
				.OnCheckStateChanged(this, &FMeshMaterialsLayout::OnOverrideUVDensityChanged, SlotIndex)
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("OverrideUVDensity", "Override UV Density"))
			]
		]
		+ GetUVDensitySlot(SlotIndex, 0)
		+ GetUVDensitySlot(SlotIndex, 1)
		+ GetUVDensitySlot(SlotIndex, 2)
		+ GetUVDensitySlot(SlotIndex, 3);
#endif
}

TSharedRef<SWidget> FMeshMaterialsLayout::OnGenerateNameWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshMaterialsLayout::IsMaterialHighlighted, SlotIndex)
			.OnCheckStateChanged(this, &FMeshMaterialsLayout::OnMaterialHighlightedChanged, SlotIndex)
			.ToolTipText(LOCTEXT("Highlight_CustomMaterialName_ToolTip", "Highlights this material in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
				.Text(LOCTEXT("Highlight", "Highlight"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 2, 0, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FMeshMaterialsLayout::IsMaterialIsolatedEnabled, SlotIndex)
			.OnCheckStateChanged(this, &FMeshMaterialsLayout::OnMaterialIsolatedChanged, SlotIndex)
			.ToolTipText(LOCTEXT("Isolate_CustomMaterialName_ToolTip", "Isolates this material in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
				.Text(LOCTEXT("Isolate", "Isolate"))
			]
		];
}

TSharedRef<SWidget> FMeshMaterialsLayout::OnGenerateMaterialListExtraBottomWidget(UMaterialInterface* Material, int32 SlotIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (!StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		return SNullWidget::NullWidget;
	}
	TSharedRef<SVerticalBox> VerticalWidget = SNew(SVerticalBox);
	const FStaticMaterial& StaticlMaterial = StaticMesh.GetStaticMaterials()[SlotIndex];
	if (GEnableStaticMeshMaterialSlotOverlayMaterialUI || StaticlMaterial.OverlayMaterialInterface)
	{
		VerticalWidget->AddSlot()
			.AutoHeight()
			.Padding(0, 2, 0, 0)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
							.HAlign(HAlign_Right)
							[
								SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.Text(LOCTEXT("OverlayMaterial", "Overlay Material"))
							]
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(5, 0, 0, 0)
					[
						SNew(SObjectPropertyEntryBox)
							.ObjectPath_Lambda([this, SlotIndex]()
								{
									UStaticMesh& StaticMesh = GetStaticMesh();
									UMaterialInterface* OverlayMaterial = nullptr;
									if (StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
									{
										OverlayMaterial = StaticMesh.GetStaticMaterials()[SlotIndex].OverlayMaterialInterface;
									}
									if (OverlayMaterial)
									{
										return OverlayMaterial->GetPathName();
									}
									return FString();
								})
							.AllowClear(true)
							.AllowedClass(UMaterialInterface::StaticClass())
							.OnObjectChanged_Lambda([this, SlotIndex](const FAssetData& AssetData)
								{
									UMaterialInterface* MaterialInterface = AssetData.IsValid() ? Cast<UMaterialInterface>(AssetData.GetAsset()) : nullptr;
									OnMaterialSlotOverlayMaterialChanged(MaterialInterface, SlotIndex);
								})
							.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
							.DisplayCompactSize(true)
					]
			];
	}

	return VerticalWidget;
}

void FMeshMaterialsLayout::OnMaterialSlotOverlayMaterialChanged(UMaterialInterface* NewOverlayMaterial, int32 SlotIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (!StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		return;
	}
	FStaticMaterial& StaticlMaterial = StaticMesh.GetStaticMaterials()[SlotIndex];

	if (StaticlMaterial.OverlayMaterialInterface == NewOverlayMaterial)
	{
		//Nothing change
		return;
	}

	//Make a transactional change
	{
		FScopedTransaction ScopeTransaction(LOCTEXT("StaticMeshSetMaterialSlotOverlayMaterialTransaction", "Staticmesh editor: Set Overlay Material For Material Slot"));

		FProperty* ChangedProperty = NULL;
		ChangedProperty = FindFProperty<FProperty>(UStaticMesh::StaticClass(), "StaticMaterials");
		check(ChangedProperty);
		StaticMesh.PreEditChange(ChangedProperty);

		StaticlMaterial.OverlayMaterialInterface = NewOverlayMaterial;

		FPropertyChangedEvent PropertyUpdateStruct(ChangedProperty);
		StaticMesh.PostEditChangeProperty(PropertyUpdateStruct);
	}
}

ECheckBoxState FMeshMaterialsLayout::IsMaterialHighlighted(int32 SlotIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		State = Component->SelectedEditorMaterial == SlotIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FMeshMaterialsLayout::OnMaterialHighlightedChanged(ECheckBoxState NewState, int32 SlotIndex)
{
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			Component->SelectedEditorMaterial = SlotIndex;
			if (Component->MaterialIndexPreview != SlotIndex)
			{
				Component->SetMaterialPreview(INDEX_NONE);
			}
			Component->SetSectionPreview(INDEX_NONE);
			Component->SelectedEditorSection = INDEX_NONE;
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			Component->SelectedEditorMaterial = INDEX_NONE;
		}
		Component->MarkRenderStateDirty();
		Component->PushSelectionToProxy();
		StaticMeshEditor.RefreshViewport();
	}
}

ECheckBoxState FMeshMaterialsLayout::IsMaterialIsolatedEnabled(int32 SlotIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		State = Component->MaterialIndexPreview == SlotIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FMeshMaterialsLayout::OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 SlotIndex)
{
	UStaticMeshComponent* Component = StaticMeshEditor.GetStaticMeshComponent();
	if (Component)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			Component->SetMaterialPreview(SlotIndex);
			if (Component->SelectedEditorMaterial != SlotIndex)
			{
				Component->SelectedEditorMaterial = INDEX_NONE;
			}
			Component->SetSectionPreview(INDEX_NONE);
			Component->SelectedEditorSection = INDEX_NONE;
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			Component->SetMaterialPreview(INDEX_NONE);
		}
		Component->MarkRenderStateDirty();
		StaticMeshEditor.RefreshViewport();
	}
}

void FMeshMaterialsLayout::OnResetMaterialToDefaultClicked(UMaterialInterface* Material, int32 MaterialIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	check(StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex));
	StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
	CallPostEditChange();
}

FText FMeshMaterialsLayout::GetOriginalImportMaterialNameText(int32 MaterialIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		FString OriginalImportMaterialName;
		StaticMesh.GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName.ToString(OriginalImportMaterialName);
		OriginalImportMaterialName = TEXT("Original Imported Material Name: ") + OriginalImportMaterialName;
		return FText::FromString(OriginalImportMaterialName);
	}
	return FText::FromName(NAME_None);
}

FText FMeshMaterialsLayout::GetMaterialNameText(int32 MaterialIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex))
	{
		return FText::FromName(StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialSlotName);
	}
	return FText::FromName(NAME_None);
}

void FMeshMaterialsLayout::OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FName InValueName = FName(*(InValue.ToString()));
	if (StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex) && StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialSlotName != InValueName)
	{
		FScopedTransaction ScopeTransaction(LOCTEXT("StaticMeshEditorMaterialSlotNameChanged", "Staticmesh editor: Material slot name change"));

		FProperty* ChangedProperty = NULL;
		ChangedProperty = FindFProperty<FProperty>(UStaticMesh::StaticClass(), "StaticMaterials");
		check(ChangedProperty);
		StaticMesh.PreEditChange(ChangedProperty);

		StaticMesh.GetStaticMaterials()[MaterialIndex].MaterialSlotName = InValueName;
		
		FPropertyChangedEvent PropertyUpdateStruct(ChangedProperty);
		StaticMesh.PostEditChangeProperty(PropertyUpdateStruct);
	}
}

bool FMeshMaterialsLayout::CanDeleteMaterialSlot(int32 MaterialIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	return StaticMesh.GetStaticMaterials().IsValidIndex(MaterialIndex);
}

void FMeshMaterialsLayout::OnDeleteMaterialSlot(int32 MaterialIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (CanDeleteMaterialSlot(MaterialIndex))
	{
		if (!bDeleteWarningConsumed)
		{
			EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("FMeshMaterialsLayout_DeleteMaterialSlot", "WARNING - Deleting a material slot can break the game play blueprint or the game play code. All indexes after the delete slot will change"));
			if (Answer == EAppReturnType::Cancel)
			{
				return;
			}
			bDeleteWarningConsumed = true;
		}

		FScopedTransaction Transaction(LOCTEXT("StaticMeshEditorDeletedMaterialSlot", "Staticmesh editor: Deleted material slot"));

		StaticMesh.Modify();
		StaticMesh.GetStaticMaterials().RemoveAt(MaterialIndex);

		//Fix the section info, the FMeshDescription use FName to retrieve the indexes when we build so no need to fix it
		for (int32 LodIndex = 0; LodIndex < StaticMesh.GetNumLODs(); ++LodIndex)
		{
			for (int32 SectionIndex = 0; SectionIndex < StaticMesh.GetNumSections(LodIndex); ++SectionIndex)
			{
				if (StaticMesh.GetSectionInfoMap().IsValidSection(LodIndex, SectionIndex))
				{
					FMeshSectionInfo SectionInfo = StaticMesh.GetSectionInfoMap().Get(LodIndex, SectionIndex);
					if (SectionInfo.MaterialIndex > MaterialIndex)
					{
						SectionInfo.MaterialIndex -= 1;
						StaticMesh.GetSectionInfoMap().Set(LodIndex, SectionIndex, SectionInfo);
					}
				}
			}
		}

		StaticMesh.PostEditChange();
	}

}

TSharedRef<SWidget> FMeshMaterialsLayout::OnGetMaterialSlotUsedByMenuContent(int32 MaterialIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	FMenuBuilder MenuBuilder(true, NULL);
	TArray<FSectionLocalizer> *SectionLocalizers;
	if (MaterialUsedMap.Contains(MaterialIndex))
	{
		SectionLocalizers = MaterialUsedMap.Find(MaterialIndex);
		FUIAction Action;
		FText EmptyTooltip;
		// Add a menu item for each texture.  Clicking on the texture will display it in the content browser
		for (const FSectionLocalizer& SectionUsingMaterial : (*SectionLocalizers))
		{
			FString ArrayItemName = TEXT("Lod ") + FString::FromInt(SectionUsingMaterial.LODIndex) + TEXT("  Index ") + FString::FromInt(SectionUsingMaterial.SectionIndex);
			MenuBuilder.AddMenuEntry(FText::FromString(ArrayItemName), EmptyTooltip, FSlateIcon(), Action);
		}
	}
	return MenuBuilder.MakeWidget();
}

FText FMeshMaterialsLayout::GetFirstMaterialSlotUsedBySection(int32 MaterialIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (MaterialUsedMap.Contains(MaterialIndex))
	{
		const TArray<FSectionLocalizer> *SectionLocalizers = MaterialUsedMap.Find(MaterialIndex);
		if (SectionLocalizers->Num() > 0)
		{
			FString ArrayItemName = FString::FromInt(SectionLocalizers->Num()) + TEXT(" Sections");
			return FText::FromString(ArrayItemName);
		}
	}
	return FText();
}

bool FMeshMaterialsLayout::OnMaterialListDirty()
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	bool ForceMaterialListRefresh = false;
	TMap<int32, TArray<FSectionLocalizer>> TempMaterialUsedMap;
	for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh.GetStaticMaterials().Num(); ++MaterialIndex)
	{
		TArray<FSectionLocalizer> SectionLocalizers;
		for (int32 LODIndex = 0; LODIndex < StaticMesh.GetNumLODs(); ++LODIndex)
		{
			for (int32 SectionIndex = 0; SectionIndex < StaticMesh.GetNumSections(LODIndex); ++SectionIndex)
			{
				FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);

				if (Info.MaterialIndex == MaterialIndex)
				{
					SectionLocalizers.Add(FSectionLocalizer(LODIndex, SectionIndex));
				}
			}
		}
		TempMaterialUsedMap.Add(MaterialIndex, SectionLocalizers);
	}
	if (TempMaterialUsedMap.Num() != MaterialUsedMap.Num())
	{
		ForceMaterialListRefresh = true;
	}
	else if (!ForceMaterialListRefresh)
	{
		for (auto KvpOld : MaterialUsedMap)
		{
			if (!TempMaterialUsedMap.Contains(KvpOld.Key))
			{
				ForceMaterialListRefresh = true;
				break;
			}
			const TArray<FSectionLocalizer> &TempSectionLocalizers = (*(TempMaterialUsedMap.Find(KvpOld.Key)));
			const TArray<FSectionLocalizer> &OldSectionLocalizers = KvpOld.Value;
			if (TempSectionLocalizers.Num() != OldSectionLocalizers.Num())
			{
				ForceMaterialListRefresh = true;
				break;
			}
			for (int32 SectionLocalizerIndex = 0; SectionLocalizerIndex < OldSectionLocalizers.Num(); ++SectionLocalizerIndex)
			{
				if (OldSectionLocalizers[SectionLocalizerIndex] != TempSectionLocalizers[SectionLocalizerIndex])
				{
					ForceMaterialListRefresh = true;
					break;
				}
			}
			if (ForceMaterialListRefresh)
			{
				break;
			}
		}
	}
	MaterialUsedMap = TempMaterialUsedMap;

	return ForceMaterialListRefresh;
}

ECheckBoxState FMeshMaterialsLayout::IsShadowCastingEnabled(int32 SlotIndex) const
{
	bool FirstEvalDone = false;
	bool ShadowCastingValue = false;
	UStaticMesh& StaticMesh = GetStaticMesh();
	for (int32 LODIndex = 0; LODIndex < StaticMesh.GetNumLODs(); ++LODIndex)
	{
		for (int32 SectionIndex = 0; SectionIndex < StaticMesh.GetNumSections(LODIndex); ++SectionIndex)
		{
			FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
			if (Info.MaterialIndex == SlotIndex)
			{
				if (!FirstEvalDone)
				{
					ShadowCastingValue = Info.bCastShadow;
					FirstEvalDone = true;
				}
				else if (ShadowCastingValue != Info.bCastShadow)
				{
					return ECheckBoxState::Undetermined;
				}
			}
		}
	}
	if (FirstEvalDone)
	{
		return ShadowCastingValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FMeshMaterialsLayout::OnShadowCastingChanged(ECheckBoxState NewState, int32 SlotIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	
	if (NewState == ECheckBoxState::Undetermined)
		return;
	
	bool CastShadow = (NewState == ECheckBoxState::Checked) ? true : false;
	bool SomethingChange = false;
	for (int32 LODIndex = 0; LODIndex < StaticMesh.GetNumLODs(); ++LODIndex)
	{
		for (int32 SectionIndex = 0; SectionIndex < StaticMesh.GetNumSections(LODIndex); ++SectionIndex)
		{
			FMeshSectionInfo Info = StaticMesh.GetSectionInfoMap().Get(LODIndex, SectionIndex);
			if (Info.MaterialIndex == SlotIndex)
			{
				Info.bCastShadow = CastShadow;
				StaticMesh.GetSectionInfoMap().Set(LODIndex, SectionIndex, Info);
				SomethingChange = true;
			}
		}
	}

	if (SomethingChange)
	{
		CallPostEditChange();
	}
}

EVisibility FMeshMaterialsLayout::GetOverrideUVDensityVisibililty() const
{
	if (StaticMeshEditor.GetViewMode() == VMI_MeshUVDensityAccuracy)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

ECheckBoxState FMeshMaterialsLayout::IsUVDensityOverridden(int32 SlotIndex) const
{
	const UStaticMesh& StaticMesh = GetStaticMesh();
	if (!StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		return ECheckBoxState::Undetermined;
	}
	else if (StaticMesh.GetStaticMaterials()[SlotIndex].UVChannelData.bOverrideDensities)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}


void FMeshMaterialsLayout::OnOverrideUVDensityChanged(ECheckBoxState NewState, int32 SlotIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (NewState != ECheckBoxState::Undetermined && StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		StaticMesh.GetStaticMaterials()[SlotIndex].UVChannelData.bOverrideDensities = (NewState == ECheckBoxState::Checked);
		StaticMesh.UpdateUVChannelData(true);
	}
}

EVisibility FMeshMaterialsLayout::GetUVDensityVisibility(int32 SlotIndex, int32 UVChannelIndex) const
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (StaticMeshEditor.GetViewMode() == VMI_MeshUVDensityAccuracy && IsUVDensityOverridden(SlotIndex) == ECheckBoxState::Checked && UVChannelIndex < StaticMeshEditor.GetNumUVChannels())
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

TOptional<float> FMeshMaterialsLayout::GetUVDensityValue(int32 SlotIndex, int32 UVChannelIndex) const
{
	const UStaticMesh& StaticMesh = GetStaticMesh();
	if (StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		float Value = StaticMesh.GetStaticMaterials()[SlotIndex].UVChannelData.LocalUVDensities[UVChannelIndex];
		return FMath::RoundToFloat(Value * 4.f) * .25f;
	}
	return TOptional<float>();
}

void FMeshMaterialsLayout::SetUVDensityValue(float InDensity, ETextCommit::Type CommitType, int32 SlotIndex, int32 UVChannelIndex)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (StaticMesh.GetStaticMaterials().IsValidIndex(SlotIndex))
	{
		StaticMesh.GetStaticMaterials()[SlotIndex].UVChannelData.LocalUVDensities[UVChannelIndex] = FMath::Max<float>(0, InDensity);
		StaticMesh.UpdateUVChannelData(true);
	}
}

void FMeshMaterialsLayout::CallPostEditChange(FProperty* PropertyChanged/*=nullptr*/)
{
	UStaticMesh& StaticMesh = GetStaticMesh();
	if (PropertyChanged)
	{
		FPropertyChangedEvent PropertyUpdateStruct(PropertyChanged);
		StaticMesh.PostEditChangeProperty(PropertyUpdateStruct);
	}
	else
	{
		StaticMesh.Modify();
		StaticMesh.PostEditChange();
	}
	if (StaticMesh.GetBodySetup())
	{
		StaticMesh.GetBodySetup()->CreatePhysicsMeshes();
	}
	StaticMeshEditor.RefreshViewport();
}


/////////////////////////////////
// FLevelOfDetailSettingsLayout
/////////////////////////////////

FLevelOfDetailSettingsLayout::FLevelOfDetailSettingsLayout( FStaticMeshEditor& InStaticMeshEditor )
	: StaticMeshEditor( InStaticMeshEditor )
{
	LODGroupNames.Reset();
	UStaticMesh::GetLODGroups(LODGroupNames);
	for (int32 GroupIndex = 0; GroupIndex < LODGroupNames.Num(); ++GroupIndex)
	{
		LODGroupOptions.Add(MakeShareable(new FString(LODGroupNames[GroupIndex].GetPlainNameString())));
	}

	for (int32 i = 0; i < MAX_STATIC_MESH_LODS; ++i)
	{
		bBuildSettingsExpanded[i] = false;
		bReductionSettingsExpanded[i] = false;
		bSectionSettingsExpanded[i] = (i == 0);

		LODScreenSizes[i] = 0.0f;
	}

	LODCount = StaticMeshEditor.GetStaticMesh()->GetNumLODs();

	UpdateLODNames();

	OnAssetPostLODImportDelegateHandle = GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostLODImport.AddLambda([this](UObject* InObject, int32 InLODIndex)
		{
			UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
			if (InObject == StaticMesh)
			{
				StaticMeshEditor.RefreshTool();
			}
		});
}

/** Returns true if automatic mesh reduction is available. */
static bool IsAutoMeshReductionAvailable()
{
	bool bAutoMeshReductionAvailable = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetStaticMeshReductionInterface() != NULL;
	return bAutoMeshReductionAvailable;
}

void FLevelOfDetailSettingsLayout::AddToDetailsPanel( IDetailLayoutBuilder& DetailBuilder )
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();

	IDetailCategoryBuilder& LODSettingsCategory =
		DetailBuilder.EditCategory( "LodSettings", LOCTEXT("LodSettingsCategory", "LOD Settings") );

	int32 LODGroupIndex = LODGroupNames.Find(StaticMesh->GetLODGroup());
	check(LODGroupIndex == INDEX_NONE || LODGroupIndex < LODGroupOptions.Num());

	IDetailPropertyRow& LODGroupRow = LODSettingsCategory.AddProperty(UStaticMesh::GetLODGroupMemberName());

	LODGroupRow.CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.Font( IDetailLayoutBuilder::GetDetailFont() )
		.Text(LOCTEXT("LODGroup", "LOD Group"))
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SAssignNew(LODGroupComboBox, STextComboBox)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OptionsSource(&LODGroupOptions)
		.InitiallySelectedItem(LODGroupOptions[(LODGroupIndex == INDEX_NONE) ? 0 : LODGroupIndex])
		.OnSelectionChanged(this, &FLevelOfDetailSettingsLayout::OnLODGroupChanged)
	];
	
	LODSettingsCategory.AddCustomRow( LOCTEXT("LODImport", "LOD Import") )
	.RowTag("LODImport")
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("LODImport", "LOD Import"))
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(STextComboBox)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OptionsSource(&LODNames)
		.InitiallySelectedItem(LODNames[0])
		.OnSelectionChanged(this, &FLevelOfDetailSettingsLayout::OnImportLOD)
	];

	TAttribute<bool> IsMinLODEnabled = TAttribute<bool>::CreateLambda([this]() { return FLevelOfDetailSettingsLayout::GetLODCount() > 1 && !GEngine->UseStaticMeshMinLODPerQualityLevels; });

	{
		TAttribute<TArray<FName>> PlatformOverrideNames = TAttribute<TArray<FName>>::CreateSP(this, &FLevelOfDetailSettingsLayout::GetMinLODPlatformOverrideNames);

		FPerPlatformPropertyCustomNodeBuilderArgs Args;
		Args.FilterText = LOCTEXT("MinLOD", "Minimum LOD");
		Args.Name = "MinLod";
		Args.OnGenerateNameWidget = FOnGetContent::CreateLambda([]()
			{
				return SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("MinLOD", "Minimum LOD"));
			}
		);

		Args.PlatformOverrideNames = PlatformOverrideNames;
		Args.OnAddPlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::AddMinLODPlatformOverride);
		Args.OnRemovePlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::RemoveMinLODPlatformOverride);
		Args.OnGenerateWidgetForPlatformRow = FOnGenerateWidget::CreateSP(this, &FLevelOfDetailSettingsLayout::GetMinLODWidget);
		Args.IsEnabled = IsMinLODEnabled;

		LODSettingsCategory.AddCustomBuilder(MakeShared<FPerPlatformPropertyCustomNodeBuilder>(MoveTemp(Args)));
	}

	TAttribute<bool> IsQualityLevelLODEnabled = TAttribute<bool>::CreateLambda([this]() { return FLevelOfDetailSettingsLayout::GetLODCount() > 1 && GEngine->UseStaticMeshMinLODPerQualityLevels; });

	LODSettingsCategory.AddCustomRow(LOCTEXT("QualityLevelMinLOD", "Quality Level Min LOD"))
	.Visibility(GEngine->UseStaticMeshMinLODPerQualityLevels ? EVisibility::Visible : EVisibility::Collapsed)
	.RowTag("QualityLevelMinLOD")
	.IsEnabled(IsQualityLevelLODEnabled)
	.EditCondition(IsQualityLevelLODEnabled, NULL)
	.NameContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(0.0f, 4.0f)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("QualityLevelMinLOD", "Quality Level Min LOD"))
			]
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(50.0f, 0.0f)
				.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FLevelOfDetailSettingsLayout::ResetToDefault)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("QualityLevelMinLodToolTip", "Clear MinLOD conversion data"))
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled(TAttribute<bool>::CreateLambda([this]() { return GetMinLOD().PerPlatform.Num() != 0 || GetMinLOD().Default != 0; }))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		]
		.ValueContent()
		.MinDesiredWidth((float)(StaticMesh->GetQualityLevelMinLOD().PerQuality.Num() + 1)*125.0f)
		.MaxDesiredWidth((float)((int32)EPerQualityLevels::Num + 1)*125.0f)
		[
			SNew(SPerQualityLevelPropertiesWidget)
			.OnGenerateWidget(this, &FLevelOfDetailSettingsLayout::GetMinQualityLevelLODWidget)
			.OnAddEntry(this, &FLevelOfDetailSettingsLayout::AddMinLODQualityLevelOverride)
			.OnRemoveEntry(this, &FLevelOfDetailSettingsLayout::RemoveMinLODQualityLevelOverride)
			.EntryNames(this, &FLevelOfDetailSettingsLayout::GetMinQualityLevelLODOverrideNames)
		];

	LODSettingsCategory.AddCustomRow(LOCTEXT("NoRefStreamingLODBias", "NoRef Streaming LOD Bias"))
	.RowTag("NoRefStreamingLODBias")
	.IsEnabled(TAttribute<bool>::CreateLambda([this]() { return GetLODCount() > 1; }))
	.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("NoRefStreamingLODBias", "NoRef Streaming LOD Bias"))
		]
		.ValueContent()
		.MinDesiredWidth(float(StaticMesh->GetNoRefStreamingLODBias().PerQuality.Num() + 1) * 125.f)
		.MaxDesiredWidth(float((int32)EPerQualityLevels::Num + 1) * 125.f)
		[
			SNew(SPerQualityLevelPropertiesWidget)
			.OnGenerateWidget(this, &FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasWidget)
			.OnAddEntry(this, &FLevelOfDetailSettingsLayout::AddNoRefStreamingLODBiasOverride)
			.OnRemoveEntry(this, &FLevelOfDetailSettingsLayout::RemoveNoRefStreamingLODBiasOverride)
			.EntryNames(this, &FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasOverrideNames)
		];

	{
		TAttribute<TArray<FName>> PlatformOverrideNames = TAttribute<TArray<FName>>::CreateSP(this, &FLevelOfDetailSettingsLayout::GetNumStreamedLODsPlatformOverrideNames);

		FPerPlatformPropertyCustomNodeBuilderArgs Args;
		Args.FilterText = LOCTEXT("NumStreamdLODs", "Num Streamed LODs");
		Args.OnGenerateNameWidget = FOnGetContent::CreateLambda([]()
			{
				return SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("NumStreamdLODs", "Num Streamed LODs"));
			}
		);

		Args.PlatformOverrideNames = PlatformOverrideNames;
		Args.OnAddPlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::AddNumStreamedLODsPlatformOverride);
		Args.OnRemovePlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::RemoveNumStreamedLODsPlatformOverride);
		Args.OnGenerateWidgetForPlatformRow = FOnGenerateWidget::CreateSP(this, &FLevelOfDetailSettingsLayout::GetNumStreamedLODsWidget);
		Args.IsEnabled = TAttribute<bool>::CreateLambda([this]() { return GetLODCount() > 1; });

		LODSettingsCategory.AddCustomBuilder(MakeShared<FPerPlatformPropertyCustomNodeBuilder>(MoveTemp(Args)));
	}

	// Add Number of LODs slider.
	const int32 MinAllowedLOD = 1;
	LODSettingsCategory.AddCustomRow( LOCTEXT("NumberOfLODs", "Number of LODs") )
	.RowTag("NumberOfLODs")
	.NameContent()
	[
		SNew(STextBlock)
		.Font( IDetailLayoutBuilder::GetDetailFont() )
		.Text(LOCTEXT("NumberOfLODs", "Number of LODs"))
	]
	.ValueContent().VAlign(VAlign_Center)
	[
		SNew(SSpinBox<int32>)
		.Font( IDetailLayoutBuilder::GetDetailFont() )
		.Value(this, &FLevelOfDetailSettingsLayout::GetLODCount)
		.OnValueChanged(this, &FLevelOfDetailSettingsLayout::OnLODCountChanged)
		.OnValueCommitted(this, &FLevelOfDetailSettingsLayout::OnLODCountCommitted)
		.MinValue(MinAllowedLOD)
		.MaxValue(MAX_STATIC_MESH_LODS)
		.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetLODCountTooltip)
		.IsEnabled(IsAutoMeshReductionAvailable())
	];

	// Auto LOD distance check box.
	LODSettingsCategory.AddCustomRow( LOCTEXT("AutoComputeLOD", "Auto Compute LOD Distances") )
	.RowTag("AutoComputeLOD")
	.NameContent()
	[
		SNew(STextBlock)
		.Font( IDetailLayoutBuilder::GetDetailFont() )
		.Text(LOCTEXT("AutoComputeLOD", "Auto Compute LOD Distances"))
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked(this, &FLevelOfDetailSettingsLayout::IsAutoLODChecked)
		.OnCheckStateChanged(this, &FLevelOfDetailSettingsLayout::OnAutoLODChanged)
	];

	LODSettingsCategory.AddCustomRow( LOCTEXT("ApplyChanges", "Apply Changes") )
	.RowTag("ApplyChanges")
	.ValueContent()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.OnClicked(this, &FLevelOfDetailSettingsLayout::OnApply)
		.IsEnabled(this, &FLevelOfDetailSettingsLayout::IsApplyNeeded)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew( STextBlock )
			.Text(LOCTEXT("ApplyChanges", "Apply Changes"))
			.Font( DetailBuilder.GetDetailFont() )
		]
	];

	AddLODLevelCategories( DetailBuilder );
}

bool FLevelOfDetailSettingsLayout::CanRemoveLOD(int32 LODIndex) const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();

	if (StaticMesh != nullptr)
	{
		const int32 NumLODs = StaticMesh->GetNumLODs();
		
		// LOD0 should never be removed
		return (NumLODs > 1 && LODIndex > 0 && LODIndex < NumLODs);
	}

	return false;
}

FReply FLevelOfDetailSettingsLayout::OnRemoveLOD(int32 LODIndex)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();

	if (StaticMesh != nullptr)
	{
		const int32 NumLODs = StaticMesh->GetNumLODs();

		if (NumLODs > 1 && LODIndex > 0 && LODIndex < NumLODs)
		{
			FText RemoveLODText = FText::Format( LOCTEXT("ConfirmRemoveLOD", "Are you sure you want to remove LOD {0} from {1}?"), LODIndex, FText::FromString(StaticMesh->GetName()) );

			if (FMessageDialog::Open(EAppMsgType::YesNo, RemoveLODText) == EAppReturnType::Yes)
			{
				FText TransactionDescription = FText::Format( LOCTEXT("OnRemoveLOD", "Staticmesh editor: Remove LOD {0}"), LODIndex);
				FScopedTransaction Transaction( TEXT(""), TransactionDescription, StaticMesh );

				StaticMesh->Modify();
				StaticMesh->RemoveSourceModel(LODIndex);
				--LODCount;
				StaticMesh->PostEditChange();

				StaticMeshEditor.RefreshTool();
			}
		}
	}

	return FReply::Handled();
}

void FLevelOfDetailSettingsLayout::AddLODLevelCategories( IDetailLayoutBuilder& DetailBuilder )
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	
	if( StaticMesh )
	{
		const int32 StaticMeshLODCount = StaticMesh->GetNumLODs();

		//Add the Materials array
		{
			FString CategoryName = FString(TEXT("StaticMeshMaterials"));

			IDetailCategoryBuilder& MaterialsCategory = DetailBuilder.EditCategory(*CategoryName, LOCTEXT("StaticMeshMaterialsLabel", "Material Slots"), ECategoryPriority::Important);
			MaterialsCategory.SetSortOrder(0);
			MaterialsLayoutWidget = MakeShareable(new FMeshMaterialsLayout(StaticMeshEditor));
			TArray<FAssetData> AssetDataArray;
			AssetDataArray.Add(FAssetData(StaticMesh, false));
			MaterialsLayoutWidget->AddToCategory(MaterialsCategory, AssetDataArray);
		}

		int32 CurrentLodIndex = 0;
		
		if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
		{
			CurrentLodIndex = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
		}
		LodCategories.Empty(StaticMeshLODCount);

		FString LODControllerCategoryName = FString(TEXT("LODCustomMode"));
		FText LODControllerString = LOCTEXT("LODCustomModeCategoryName", "LOD Picker");

		IDetailCategoryBuilder& LODCustomModeCategory = DetailBuilder.EditCategory( *LODControllerCategoryName, LODControllerString, ECategoryPriority::Important );
		LodCustomCategory = &LODCustomModeCategory;

		LODCustomModeCategory.AddCustomRow((LOCTEXT("LODCustomModeSelect", "Select LOD")))
		.RowTag("SelectLOD")
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LODCustomModeSelectTitle", "LOD"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(this, &FLevelOfDetailSettingsLayout::IsLodComboBoxEnabledForLodPicker)
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			OnGenerateLodComboBoxForLodPicker()
		];

		LODCustomModeCategory.AddCustomRow((LOCTEXT("LODCustomModeFirstRowName", "LODCustomMode")))
		.RowTag("LODCustomMode")
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FLevelOfDetailSettingsLayout::GetLODCustomModeNameContent, (int32)INDEX_NONE)
			.ToolTipText(LOCTEXT("LODCustomModeFirstRowTooltip", "Custom Mode shows multiple LOD's properties at the same time for easier editing."))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FLevelOfDetailSettingsLayout::IsLODCustomModeCheck, (int32)INDEX_NONE)
			.OnCheckStateChanged(this, &FLevelOfDetailSettingsLayout::SetLODCustomModeCheck, (int32)INDEX_NONE)
			.ToolTipText(LOCTEXT("LODCustomModeFirstRowTooltip", "Custom Mode shows multiple LOD's properties at the same time for easier editing."))
		];
		// Create information panel for each LOD level.
		for(int32 LODIndex = 0; LODIndex < StaticMeshLODCount; ++LODIndex)
		{
			//Show the viewport LOD at start
			bool IsViewportLOD = (CurrentLodIndex == 0 ? 0 : CurrentLodIndex - 1) == LODIndex;
			DetailDisplayLODs[LODIndex] = true; //enable all LOD in custom mode
			LODCustomModeCategory.AddCustomRow((LOCTEXT("LODCustomModeRowName", "LODCheckBoxRowName")), true)
			.RowTag("LODCheckBoxRowName")
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FLevelOfDetailSettingsLayout::GetLODCustomModeNameContent, LODIndex)
				.IsEnabled(this, &FLevelOfDetailSettingsLayout::IsLODCustomModeEnable, LODIndex)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FLevelOfDetailSettingsLayout::IsLODCustomModeCheck, LODIndex)
				.OnCheckStateChanged(this, &FLevelOfDetailSettingsLayout::SetLODCustomModeCheck, LODIndex)
				.IsEnabled(this, &FLevelOfDetailSettingsLayout::IsLODCustomModeEnable, LODIndex)
			];

			if (IsAutoMeshReductionAvailable())
			{
				ReductionSettingsWidgets[LODIndex] = MakeShareable( new FMeshReductionSettingsLayout(AsShared(), LODIndex, StaticMesh->IsMeshDescriptionValid(LODIndex)));
			}

			if (LODIndex < StaticMesh->GetNumSourceModels())
			{
				const FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
				if (ReductionSettingsWidgets[LODIndex].IsValid())
				{
					ReductionSettingsWidgets[LODIndex]->UpdateSettings(SrcModel.ReductionSettings);
				}

				if (StaticMesh->IsMeshDescriptionValid(LODIndex))
				{
					BuildSettingsWidgets[LODIndex] = MakeShareable( new FMeshBuildSettingsLayout( AsShared(), LODIndex ) );
					BuildSettingsWidgets[LODIndex]->UpdateSettings(SrcModel.BuildSettings);
				}

				LODScreenSizes[LODIndex] = SrcModel.ScreenSize;
			}
			else if (LODIndex > 0)
			{
				if (ReductionSettingsWidgets[LODIndex].IsValid() && ReductionSettingsWidgets[LODIndex-1].IsValid())
				{
					FMeshReductionSettings ReductionSettings = ReductionSettingsWidgets[LODIndex-1]->GetSettings();
					// By default create LODs with half the triangles of the previous LOD.
					ReductionSettings.PercentTriangles *= 0.5f;
					ReductionSettingsWidgets[LODIndex]->UpdateSettings(ReductionSettings);
				}

				if(LODScreenSizes[LODIndex].Default >= LODScreenSizes[LODIndex-1].Default)
				{
					const float DefaultScreenSizeDifference = 0.01f;
					LODScreenSizes[LODIndex].Default = LODScreenSizes[LODIndex-1].Default - DefaultScreenSizeDifference;
				}
			}

			FString CategoryName = FString(TEXT("LOD"));
			CategoryName.AppendInt( LODIndex );

			FText LODLevelString = FText::FromString(FString(TEXT("LOD ")) + FString::FromInt(LODIndex) );
			bool bHasBeenSimplified = !StaticMesh->IsMeshDescriptionValid(LODIndex) || StaticMesh->IsReductionActive(LODIndex);
			FText GeneratedString = FText::FromString(bHasBeenSimplified ? TEXT("[generated]") : TEXT(""));

			IDetailCategoryBuilder& LODCategory = DetailBuilder.EditCategory( *CategoryName, LODLevelString, ECategoryPriority::Important );
			LodCategories.Add(&LODCategory);

			LODCategory.HeaderContent
			(
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(GeneratedString)
						.Font(IDetailLayoutBuilder::GetDetailFontItalic())
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew( SBox )
					.HAlign( HAlign_Right )
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						.Padding(FMargin(5.0f, 0.0f))
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(this, &FLevelOfDetailSettingsLayout::GetLODScreenSizeTitle, LODIndex)
							.Visibility( LODIndex > 0 ? EVisibility::Visible : EVisibility::Collapsed )
						]
						+ SHorizontalBox::Slot()
						.Padding( FMargin( 5.0f, 0.0f ) )
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text( FText::Format( LOCTEXT("Triangles_MeshSimplification", "Triangles: {0}"), FText::AsNumber( StaticMeshEditor.GetNumTriangles(LODIndex) ) ) )
						]
						+ SHorizontalBox::Slot()
						.Padding( FMargin( 5.0f, 0.0f ) )
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text( FText::Format( LOCTEXT("Vertices_MeshSimplification", "Vertices: {0}"), FText::AsNumber( StaticMeshEditor.GetNumVertices(LODIndex) ) ) )
						]
					]
				]
			);
					
			SectionSettingsWidgets[ LODIndex ] = MakeShareable( new FMeshSectionSettingsLayout( StaticMeshEditor, LODIndex, LodCategories) );
			SectionSettingsWidgets[ LODIndex ]->AddToCategory( LODCategory );

			int32 PlatformNumber = PlatformInfo::GetAllPlatformGroupNames().Num();

			TAttribute<TArray<FName>> PlatformOverrideNames = TAttribute<TArray<FName>>::Create(TAttribute<TArray<FName>>::FGetter::CreateSP(this, &FLevelOfDetailSettingsLayout::GetLODScreenSizePlatformOverrideNames, LODIndex));

			FPerPlatformPropertyCustomNodeBuilderArgs Args;
			{
				FText ScreenSizePropertyText(LOCTEXT("ScreenSizeName", "Screen Size"));
				TAttribute<bool> IsScreenSizeEnabled = TAttribute<bool>::CreateSP(this, &FLevelOfDetailSettingsLayout::CanChangeLODScreenSize);

				Args.Name = FName("ScreenSize");
				Args.FilterText = ScreenSizePropertyText;
				Args.PlatformOverrideNames = PlatformOverrideNames;
				Args.OnAddPlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::AddLODScreenSizePlatformOverride, LODIndex);
				Args.OnRemovePlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FLevelOfDetailSettingsLayout::RemoveLODScreenSizePlatformOverride, LODIndex);
				Args.OnGenerateWidgetForPlatformRow = FOnGenerateWidget::CreateSP(this, &FLevelOfDetailSettingsLayout::GetLODScreenSizeWidget, LODIndex);
				Args.IsEnabled = IsScreenSizeEnabled;
				Args.OnGenerateNameWidget = FOnGetContent::CreateLambda([IsScreenSizeEnabled = MoveTemp(IsScreenSizeEnabled), ScreenSizePropertyText = MoveTemp(ScreenSizePropertyText)]()
					{
						return SNew(STextBlock)
							.IsEnabled(IsScreenSizeEnabled)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(ScreenSizePropertyText);
					});
			}

			LODCategory.AddCustomBuilder(MakeShared<FPerPlatformPropertyCustomNodeBuilder>(MoveTemp(Args)));

			if(LODIndex > 0 && StaticMesh->IsMeshDescriptionValid(LODIndex))
			{
				FString FileTypeFilter = TEXT("All files (*.*)|*.*");
				LODCategory.AddCustomRow(( LOCTEXT("SourceImporFilenameRow", "SourceImportFilename")))
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("SourceImportFilenameName", "Source Import Filename"))
				]
				.ValueContent()
					.VAlign(VAlign_Center)
					.MinDesiredWidth(125.0f)
					.MaxDesiredWidth(0.0f)
				[
					SNew(SFilePathPicker)
						.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
						.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a source import file"))
						.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
						.BrowseTitle(LOCTEXT("PropertyEditorTitle", "Source import file picker..."))
						.FilePath(this, &FLevelOfDetailSettingsLayout::GetSourceImportFilename, LODIndex)
						.FileTypeFilter(FileTypeFilter)
						.OnPathPicked(this, &FLevelOfDetailSettingsLayout::SetSourceImportFilename, LODIndex)
				];
			}

			if (BuildSettingsWidgets[LODIndex].IsValid())
			{
				LODCategory.AddCustomBuilder( BuildSettingsWidgets[LODIndex].ToSharedRef() );
			}

			if( ReductionSettingsWidgets[LODIndex].IsValid() )
			{
				LODCategory.AddCustomBuilder( ReductionSettingsWidgets[LODIndex].ToSharedRef() );
			}

			if (LODIndex != 0)
			{
				LODCategory.AddCustomRow( LOCTEXT("RemoveLOD", "Remove LOD") )
				.ValueContent()
				.HAlign(HAlign_Left)
				[
 					SNew(SButton)
					.OnClicked(this, &FLevelOfDetailSettingsLayout::OnRemoveLOD, LODIndex)
					.IsEnabled(this, &FLevelOfDetailSettingsLayout::CanRemoveLOD, LODIndex)
					.ToolTipText( LOCTEXT("RemoveLOD_ToolTip", "Removes this LOD from the Static Mesh") )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("RemoveLOD", "Remove LOD") )
						.Font( DetailBuilder.GetDetailFont() )
					]
				];
			}
			LODCategory.SetCategoryVisibility(IsViewportLOD);
		}

		//Show the LOD custom category 
		if (StaticMeshLODCount > 1)
		{
			LODCustomModeCategory.SetCategoryVisibility(true);
			LODCustomModeCategory.SetShowAdvanced(false);
		}



		//Restore the state of the custom check LOD
		for (int32 DetailLODIndex = 0; DetailLODIndex < StaticMeshLODCount; ++DetailLODIndex)
		{
			int32 LodCheckValue = StaticMeshEditor.GetCustomData(CustomDataKey_LODVisibilityState + DetailLODIndex);
			if (LodCheckValue != INDEX_NONE)
			{
				DetailDisplayLODs[DetailLODIndex] = LodCheckValue > 0;
			}
		}

		//Restore the state of the custom LOD mode if its true (greater then 0)
		bool bCustomLodEditMode = StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0;
		if (bCustomLodEditMode)
		{
			for (int32 DetailLODIndex = 0; DetailLODIndex < StaticMeshLODCount; ++DetailLODIndex)
			{
				if (!LodCategories.IsValidIndex(DetailLODIndex))
				{
					break;
				}
				LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailDisplayLODs[DetailLODIndex]);
			}
		}

		if (LodCustomCategory != nullptr)
		{
			LodCustomCategory->SetShowAdvanced(bCustomLodEditMode);
		}
	}
}


FLevelOfDetailSettingsLayout::~FLevelOfDetailSettingsLayout()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostLODImport.Remove(OnAssetPostLODImportDelegateHandle);
}

FString FLevelOfDetailSettingsLayout::GetSourceImportFilename(int32 LODIndex) const
{
	UStaticMesh* Mesh = StaticMeshEditor.GetStaticMesh();
	if (!Mesh->IsSourceModelValid(LODIndex) || Mesh->GetSourceModel(LODIndex).SourceImportFilename.IsEmpty())
	{
		return FString(TEXT(""));
	}
	return UAssetImportData::ResolveImportFilename(Mesh->GetSourceModel(LODIndex).SourceImportFilename, nullptr);
}

void FLevelOfDetailSettingsLayout::SetSourceImportFilename(const FString& SourceFileName, int32 LODIndex) const
{
	UStaticMesh* Mesh = StaticMeshEditor.GetStaticMesh();
	if (!Mesh->IsSourceModelValid(LODIndex))
	{
		return;
	}
	if (SourceFileName.IsEmpty())
	{
		Mesh->GetSourceModel(LODIndex).SourceImportFilename = SourceFileName;
	}
	else
	{
		Mesh->GetSourceModel(LODIndex).SourceImportFilename = UAssetImportData::SanitizeImportFilename(SourceFileName, nullptr);
	}
	Mesh->Modify();
}

int32 FLevelOfDetailSettingsLayout::GetLODCount() const
{
	return LODCount;
}

float FLevelOfDetailSettingsLayout::GetLODScreenSize(FName PlatformGroupName, int32 LODIndex) const
{
	check(LODIndex < MAX_STATIC_MESH_LODS);
	UStaticMesh* Mesh = StaticMeshEditor.GetStaticMesh();
	const FPerPlatformFloat& LODScreenSize = LODScreenSizes[FMath::Clamp(LODIndex, 0, MAX_STATIC_MESH_LODS - 1)];
	float ScreenSize = LODScreenSize.Default;
	if (PlatformGroupName != NAME_None)
	{
		const float* PlatformScreenSize = LODScreenSize.PerPlatform.Find(PlatformGroupName);
		if (PlatformScreenSize != nullptr)
		{
			ScreenSize = *PlatformScreenSize;
		}
	}

	if(Mesh->GetAutoComputeLODScreenSize() && Mesh->GetRenderData())
	{
		ScreenSize = Mesh->GetRenderData()->ScreenSize[LODIndex].Default;
		const float* PlatformScreenSize = Mesh->GetRenderData()->ScreenSize[LODIndex].PerPlatform.Find(PlatformGroupName);
		if (PlatformScreenSize != nullptr)
		{
			ScreenSize = *PlatformScreenSize;
		}

	}
	else if(Mesh->IsSourceModelValid(LODIndex))
	{
		ScreenSize = Mesh->GetSourceModel(LODIndex).ScreenSize.Default;
		const float* PlatformScreenSize = Mesh->GetSourceModel(LODIndex).ScreenSize.PerPlatform.Find(PlatformGroupName);
		if (PlatformScreenSize != nullptr)
		{
			ScreenSize = *PlatformScreenSize;
		}
	}
	return ScreenSize;
}

FText FLevelOfDetailSettingsLayout::GetLODScreenSizeTitle( int32 LODIndex ) const
{
	return FText::Format( LOCTEXT("ScreenSize_MeshSimplification", "Screen Size: {0}"), FText::AsNumber(GetLODScreenSize(NAME_None, LODIndex)));
}

bool FLevelOfDetailSettingsLayout::CanChangeLODScreenSize() const
{
	return !IsAutoLODEnabled();
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::GetLODScreenSizeWidget(FName PlatformGroupName, int32 LODIndex) const
{
	return SNew(SSpinBox<float>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinDesiredWidth(60.0f)
		.MinValue(0.0f)
		.MaxValue(static_cast<float>(WORLD_MAX))
		.SliderExponent(2.0f)
		.Value(this, &FLevelOfDetailSettingsLayout::GetLODScreenSize, PlatformGroupName, LODIndex)
		.OnValueChanged(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnLODScreenSizeChanged, PlatformGroupName, LODIndex)
		.OnValueCommitted(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnLODScreenSizeCommitted, PlatformGroupName, LODIndex)
		.IsEnabled(this, &FLevelOfDetailSettingsLayout::CanChangeLODScreenSize);
}

TArray<FName> FLevelOfDetailSettingsLayout::GetLODScreenSizePlatformOverrideNames(int32 LODIndex) const
{
	TArray<FName> KeyArray;
	LODScreenSizes[LODIndex].PerPlatform.GenerateKeyArray(KeyArray);
	KeyArray.Sort(FNameLexicalLess());
	return KeyArray;
}

float FLevelOfDetailSettingsLayout::GetScreenSizeWidgetWidth(int32 LODIndex) const
{
	return (float)(LODScreenSizes[LODIndex].PerPlatform.Num() + 1) * 125.f;
}

bool FLevelOfDetailSettingsLayout::AddLODScreenSizePlatformOverride(FName PlatformGroupName, int32 LODIndex)
{
	FScopedTransaction Transaction(LOCTEXT("AddLODScreenSizePlatformOverride", "Add LOD Screen Size Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	if (LODScreenSizes[LODIndex].PerPlatform.Find(PlatformGroupName) == nullptr)
	{
		if(!StaticMesh->GetAutoComputeLODScreenSize() && StaticMesh->IsSourceModelValid(LODIndex))
		{
			StaticMesh->Modify();
			float Value = StaticMesh->GetSourceModel(LODIndex).ScreenSize.Default;
			StaticMesh->GetSourceModel(LODIndex).ScreenSize.PerPlatform.Add(PlatformGroupName, Value);
			OnLODScreenSizeChanged(Value, PlatformGroupName, LODIndex);
			return true;
		}
	}
	return false;
}

bool FLevelOfDetailSettingsLayout::RemoveLODScreenSizePlatformOverride(FName PlatformGroupName, int32 LODIndex)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveLODScreenSizePlatformOverride", "Remove LOD Screen Size Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	if (!StaticMesh->GetAutoComputeLODScreenSize() && StaticMesh->IsSourceModelValid(LODIndex))
	{
		StaticMesh->Modify();
		if (StaticMesh->GetSourceModel(LODIndex).ScreenSize.PerPlatform.Remove(PlatformGroupName) != 0)
		{
			OnLODScreenSizeChanged(StaticMesh->GetSourceModel(LODIndex).ScreenSize.Default, PlatformGroupName, LODIndex);
			return true;
		}
	}
	return false;
}

void FLevelOfDetailSettingsLayout::OnLODScreenSizeChanged( float NewValue, FName PlatformGroupName, int32 LODIndex )
{
	check(LODIndex < MAX_STATIC_MESH_LODS);
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	if (!StaticMesh->GetAutoComputeLODScreenSize())
	{
		// First propagate any changes from the source models to our local scratch.
		for (int32 i = 0; i < StaticMesh->GetNumSourceModels(); ++i)
		{
			LODScreenSizes[i] = StaticMesh->GetSourceModel(i).ScreenSize;
		}

		// Update Display factors for further LODs
		const float MinimumDifferenceInScreenSize = KINDA_SMALL_NUMBER;
		
		if (PlatformGroupName == NAME_None)
		{
			LODScreenSizes[LODIndex].Default = NewValue;

			// Make sure we aren't trying to overlap or have more than one LOD for a value
			for (int32 i = 1; i < MAX_STATIC_MESH_LODS; ++i)
			{
				float MaxValue = FMath::Max(LODScreenSizes[i-1].Default - MinimumDifferenceInScreenSize, 0.0f);
				LODScreenSizes[i].Default = FMath::Min(LODScreenSizes[i].Default, MaxValue);
			}
		}
		else
		{
			// Per-platform overrides don't have any restrictions
			float* PlatformScreenSize = LODScreenSizes[LODIndex].PerPlatform.Find(PlatformGroupName);
			if (PlatformScreenSize != nullptr)
			{
				*PlatformScreenSize = NewValue;
			}
		}

		// Push changes immediately.
		for (int32 i = 0; i < MAX_STATIC_MESH_LODS; ++i)
		{
			if (StaticMesh->IsSourceModelValid(i))
			{
				StaticMesh->GetSourceModel(i).ScreenSize = LODScreenSizes[i];
			}
			if (StaticMesh->GetRenderData()
				&& StaticMesh->GetRenderData()->LODResources.IsValidIndex(i))
			{
				StaticMesh->GetRenderData()->ScreenSize[i] = LODScreenSizes[i];
			}
		}

		// Reregister static mesh components using this mesh.
		{
			FStaticMeshComponentRecreateRenderStateContext ReregisterContext(StaticMesh,false);
			StaticMesh->Modify();
		}

		StaticMeshEditor.RefreshViewport();
	}
}

void FLevelOfDetailSettingsLayout::OnLODScreenSizeCommitted( float NewValue, ETextCommit::Type CommitType, FName PlatformGroupName, int32 LODIndex )
{
	OnLODScreenSizeChanged(NewValue, PlatformGroupName, LODIndex);
}

void FLevelOfDetailSettingsLayout::UpdateLODNames()
{
	LODNames.Empty();
	LODNames.Add( MakeShareable( new FString( LOCTEXT("BaseLOD", "LOD 0").ToString() ) ) );
	for(int32 LODLevelID = 1; LODLevelID < LODCount; ++LODLevelID)
	{
		LODNames.Add( MakeShareable( new FString( FText::Format( NSLOCTEXT("LODSettingsLayout", "LODLevel_Reimport", "Reimport LOD Level {0}"), FText::AsNumber( LODLevelID ) ).ToString() ) ) );
	}
	LODNames.Add( MakeShareable( new FString( FText::Format( NSLOCTEXT("LODSettingsLayout", "LODLevel_Import", "Import LOD Level {0}"), FText::AsNumber( LODCount ) ).ToString() ) ) );
}

void FLevelOfDetailSettingsLayout::OnBuildSettingsExpanded(bool bIsExpanded, int32 LODIndex)
{
	check(LODIndex >= 0 && LODIndex < MAX_STATIC_MESH_LODS);
	bBuildSettingsExpanded[LODIndex] = bIsExpanded;
}

void FLevelOfDetailSettingsLayout::OnReductionSettingsExpanded(bool bIsExpanded, int32 LODIndex)
{
	check(LODIndex >= 0 && LODIndex < MAX_STATIC_MESH_LODS);
	bReductionSettingsExpanded[LODIndex] = bIsExpanded;
}

void FLevelOfDetailSettingsLayout::OnSectionSettingsExpanded(bool bIsExpanded, int32 LODIndex)
{
	check(LODIndex >= 0 && LODIndex < MAX_STATIC_MESH_LODS);
	bSectionSettingsExpanded[LODIndex] = bIsExpanded;
}

void FLevelOfDetailSettingsLayout::OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	int32 GroupIndex = LODGroupOptions.Find(NewValue);
	FName NewGroup = LODGroupNames[GroupIndex];
	if (StaticMesh->GetLODGroup() != NewGroup)
	{
		if (NewGroup != NAME_None)
		{
			EAppReturnType::Type DialogResult = FMessageDialog::Open(
				EAppMsgType::YesNo,
				FText::Format(LOCTEXT("ApplyDefaultLODSettings", "Changing LOD group will overwrite the current settings with the defaults from LOD group '{0}'. Do you wish to continue?"), FText::FromString(**NewValue))
			);
			if (DialogResult == EAppReturnType::Yes)
			{
				StaticMesh->SetLODGroup(NewGroup);
				// update the internal count
				LODCount = StaticMesh->GetNumSourceModels();
				StaticMeshEditor.RefreshTool();
			}
			else
			{
				// Overriding the selection; ensure that the widget correctly reflects the property value
				int32 Index = LODGroupNames.Find(StaticMesh->GetLODGroup());
				check(Index != INDEX_NONE);
				LODGroupComboBox->SetSelectedItem(LODGroupOptions[Index]);
			}
		}
		else
		{
			//Setting to none just change the LODGroup to None, the LOD count will not change
			StaticMesh->SetLODGroup(NewGroup);
			StaticMeshEditor.RefreshTool();
		}
	}
}

bool FLevelOfDetailSettingsLayout::IsAutoLODEnabled() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	return StaticMesh->GetAutoComputeLODScreenSize();
}

ECheckBoxState FLevelOfDetailSettingsLayout::IsAutoLODChecked() const
{
	return IsAutoLODEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLevelOfDetailSettingsLayout::OnAutoLODChanged(ECheckBoxState NewState)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	StaticMesh->SetAutoComputeLODScreenSize((NewState == ECheckBoxState::Checked) ? true : false);
	if (!StaticMesh->GetAutoComputeLODScreenSize())
	{
		if (StaticMesh->GetNumSourceModels() > 0)
		{
			StaticMesh->GetSourceModel(0).ScreenSize.Default = 1.0f;
		}
		for (int32 LODIndex = 1; LODIndex < StaticMesh->GetNumSourceModels(); ++LODIndex)
		{
			StaticMesh->GetSourceModel(LODIndex).ScreenSize.Default = StaticMesh->GetRenderData()->ScreenSize[LODIndex].Default;
		}
	}
	StaticMesh->PostEditChange();
	StaticMeshEditor.RefreshTool();
}

void FLevelOfDetailSettingsLayout::OnImportLOD(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	int32 LODIndex = 0;
	if( LODNames.Find(NewValue, LODIndex) && LODIndex > 0 )
	{
		UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
		check(StaticMesh);

		if (StaticMesh->GetLODGroup() != NAME_None && StaticMesh->IsSourceModelValid(LODIndex))
		{
			// Cache derived data for the running platform.
			ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
			ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
			check(RunningPlatform);
			const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();
			const FStaticMeshLODGroup& LODGroup = LODSettings.GetLODGroup(StaticMesh->GetLODGroup());
			if (LODIndex < LODGroup.GetDefaultNumLODs())
			{
				//Ask the user to change the LODGroup to None, if the user cancel do not re-import the LOD
				//We can have a LODGroup with custom LOD only if custom LOD are after the generated LODGroup LODs
				EAppReturnType::Type ReturnResult = FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Ok, 
					FText::Format(LOCTEXT("LODImport_LODGroupVersusCustomLODConflict",
										  "This static mesh uses the LOD group \"{0}\" which controls generated LODs. Continuing this process will reset the LOD group, clear all existing LODs, and import the selected file."),
										  FText::FromName(StaticMesh->GetLODGroup())));
				if (ReturnResult == EAppReturnType::Cancel)
				{
					StaticMeshEditor.RefreshTool();
					return;
				}
				//Clear the LODGroup
				StaticMesh->SetLODGroup(NAME_None, false);
				//Make sure the importdata point on LOD Group None
				UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(StaticMesh->GetAssetImportData());
				if (ImportData != nullptr)
				{
					ImportData->StaticMeshLODGroup = NAME_None;
				}
			}
		}

		//Are we a new imported LOD, we want to set some value for new imported LOD.
		//This boolean prevent changing the value when the LOD is reimport
		bool bImportCustomLOD = (LODIndex >= StaticMesh->GetNumSourceModels());

		FbxMeshUtils::ImportMeshLODDialog(StaticMesh, LODIndex).Then([this, StaticMesh, bImportCustomLOD, LODIndex](TFuture<bool> FutureResult)
			{
				bool bResult = FutureResult.Get();
				if (bImportCustomLOD && bResult && StaticMesh->IsSourceModelValid(LODIndex))
				{
					//Custom LOD should reduce base on them self when they get imported.
					StaticMesh->GetSourceModel(LODIndex).ReductionSettings.BaseLODModel = LODIndex;
				}

				StaticMesh->PostEditChange();
				StaticMeshEditor.RefreshTool();
			});
	}

}

bool FLevelOfDetailSettingsLayout::IsApplyNeeded() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	if (StaticMesh->GetNumSourceModels() != LODCount)
	{
		return true;
	}

	for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
		if (BuildSettingsWidgets[LODIndex].IsValid()
			&& SrcModel.BuildSettings != BuildSettingsWidgets[LODIndex]->GetSettings())
		{
			return true;
		}
		if (ReductionSettingsWidgets[LODIndex].IsValid()
			&& SrcModel.ReductionSettings != ReductionSettingsWidgets[LODIndex]->GetSettings())
		{
			return true;
		}
	}

	// Allow to rebuild mesh card representation on demand when debugging it
	if (MeshCardRepresentation::IsDebugMode())
	{
		return true;
	}

	return false;
}

void FLevelOfDetailSettingsLayout::ApplyChanges()
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	// Calling Begin and EndSlowTask are rather dangerous because they tick
	// Slate. Call them here and flush rendering commands to be sure!.

	FFormatNamedArguments Args;
	Args.Add( TEXT("StaticMeshName"), FText::FromString( StaticMesh->GetName() ) );
	GWarn->BeginSlowTask( FText::Format( LOCTEXT("ApplyLODChanges", "Applying changes to {StaticMeshName}..."), Args ), true );
	FlushRenderingCommands();

	StaticMesh->Modify();
	StaticMesh->SetNumSourceModels(LODCount);

	for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		FStaticMeshSourceModel& SrcModel = StaticMesh->GetSourceModel(LODIndex);
		if (BuildSettingsWidgets[LODIndex].IsValid())
		{
			SrcModel.BuildSettings = BuildSettingsWidgets[LODIndex]->GetSettings();
		}
		if (ReductionSettingsWidgets[LODIndex].IsValid())
		{
			SrcModel.ReductionSettings = ReductionSettingsWidgets[LODIndex]->GetSettings();
		}

		if (LODIndex == 0)
		{
			SrcModel.ScreenSize.Default = 1.0f;
		}
		else
		{
			SrcModel.ScreenSize = LODScreenSizes[LODIndex];
			FStaticMeshSourceModel& PrevModel = StaticMesh->GetSourceModel(LODIndex-1);
			if(SrcModel.ScreenSize.Default >= PrevModel.ScreenSize.Default)
			{
				const float DefaultScreenSizeDifference = 0.01f;
				LODScreenSizes[LODIndex].Default = LODScreenSizes[LODIndex-1].Default - DefaultScreenSizeDifference;

				// Make sure there are no incorrectly overlapping values
				SrcModel.ScreenSize.Default = 1.0f - 0.01f * LODIndex;
			}
		}
	}
	StaticMesh->PostEditChange();

	GWarn->EndSlowTask();

	StaticMeshEditor.RefreshTool();
}

FReply FLevelOfDetailSettingsLayout::OnApply()
{
	ApplyChanges();
	return FReply::Handled();
}

void FLevelOfDetailSettingsLayout::OnLODCountChanged(int32 NewValue)
{
	LODCount = FMath::Clamp<int32>(NewValue, 1, MAX_STATIC_MESH_LODS);

	UpdateLODNames();
}

void FLevelOfDetailSettingsLayout::OnLODCountCommitted(int32 InValue, ETextCommit::Type CommitInfo)
{
	OnLODCountChanged(InValue);
}

FText FLevelOfDetailSettingsLayout::GetLODCountTooltip() const
{
	if(IsAutoMeshReductionAvailable())
	{
		return LOCTEXT("LODCountTooltip", "The number of LODs for this static mesh. If auto mesh reduction is available, setting this number will determine the number of LOD levels to auto generate.");
	}

	return LOCTEXT("LODCountTooltip_Disabled", "Auto mesh reduction is unavailable! Please provide a mesh reduction interface such as Simplygon to use this feature or manually import LOD levels.");
}

int32 FLevelOfDetailSettingsLayout::GetMinLOD(FName Platform) const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	const int32* ValuePtr = (Platform == NAME_None) ? nullptr : StaticMesh->GetMinLOD().PerPlatform.Find(Platform);
	return (ValuePtr != nullptr) ? *ValuePtr : StaticMesh->GetMinLOD().Default;
}

FPerPlatformInt FLevelOfDetailSettingsLayout::GetMinLOD() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	return StaticMesh->GetMinLOD();
}

void FLevelOfDetailSettingsLayout::OnMinLODChanged(int32 NewValue, FName Platform)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	{
		FStaticMeshComponentRecreateRenderStateContext ReregisterContext(StaticMesh, false);
		NewValue = FMath::Clamp<int32>(NewValue, 0, MAX_STATIC_MESH_LODS - 1);
		FPerPlatformInt MinLOD = StaticMesh->GetMinLOD();
		if (Platform == NAME_None)
		{
			MinLOD.Default = NewValue;
		}
		else
		{
			int32* ValuePtr = MinLOD.PerPlatform.Find(Platform);
			if (ValuePtr != nullptr)
			{
				*ValuePtr = NewValue;
			}
		}
		StaticMesh->SetMinLOD(MoveTemp(MinLOD));
		StaticMesh->Modify();
	}
	StaticMeshEditor.RefreshViewport();
}

void FLevelOfDetailSettingsLayout::OnMinLODCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName Platform)
{
	OnMinLODChanged(InValue, Platform);
}

FText FLevelOfDetailSettingsLayout::GetMinLODTooltip() const
{
	return LOCTEXT("MinLODTooltip", "The minimum LOD to use for rendering.  This can be overridden in components.");
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::GetMinLODWidget(FName PlatformGroupName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FLevelOfDetailSettingsLayout::GetMinLOD, PlatformGroupName)
		.OnValueChanged(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnMinLODChanged, PlatformGroupName)
		.OnValueCommitted(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnMinLODCommitted, PlatformGroupName)
		.MinValue(0)
		.MaxValue(MAX_STATIC_MESH_LODS)
		.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetMinLODTooltip)
		.IsEnabled(FLevelOfDetailSettingsLayout::GetLODCount() > 1);
}

bool FLevelOfDetailSettingsLayout::AddMinLODPlatformOverride(FName PlatformGroupName)
{
	FScopedTransaction Transaction(LOCTEXT("AddMinLODPlatformOverride", "Add Min LOD Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	if (StaticMesh->GetMinLOD().PerPlatform.Find(PlatformGroupName) == nullptr)
	{
		FPerPlatformInt MinLOD = StaticMesh->GetMinLOD();
		int32 Value = MinLOD.Default;
		MinLOD.PerPlatform.Add(PlatformGroupName, Value);
		StaticMesh->SetMinLOD(MoveTemp(MinLOD));
		OnMinLODChanged(Value, PlatformGroupName);
		return true;
	}
	return false;
}

bool FLevelOfDetailSettingsLayout::RemoveMinLODPlatformOverride(FName PlatformGroupName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveMinLODPlatformOverride", "Remove Min LOD Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	FPerPlatformInt MinLOD = StaticMesh->GetMinLOD();
	if (MinLOD.PerPlatform.Remove(PlatformGroupName) != 0)
	{
		int32 Value = MinLOD.Default;
		StaticMesh->SetMinLOD(MoveTemp(MinLOD));
		OnMinLODChanged(Value, PlatformGroupName);
		return true;
	}
	return false;
}

TArray<FName> FLevelOfDetailSettingsLayout::GetMinLODPlatformOverrideNames() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	TArray<FName> KeyArray;
	StaticMesh->GetMinLOD().PerPlatform.GenerateKeyArray(KeyArray);
	KeyArray.Sort(FNameLexicalLess());
	return KeyArray;
}

int32 FLevelOfDetailSettingsLayout::GetMinQualityLevelLOD(FName QualityLevel) const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
	const int32* ValuePtr = (QualityLevel == NAME_None) ? nullptr : StaticMesh->GetQualityLevelMinLOD().PerQuality.Find(QLKey);
	return (ValuePtr != nullptr) ? *ValuePtr : StaticMesh->GetQualityLevelMinLOD().Default;
}

void FLevelOfDetailSettingsLayout::OnMinQualityLevelLODChanged(int32 NewValue, FName QualityLevel)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	{
		FStaticMeshComponentRecreateRenderStateContext ReregisterContext(StaticMesh, false);
		NewValue = FMath::Clamp<int32>(NewValue, 0, MAX_STATIC_MESH_LODS - 1);
		FPerQualityLevelInt MinLOD = StaticMesh->GetQualityLevelMinLOD();
		int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
		if (QualityLevel == NAME_None || QLKey == INDEX_NONE)
		{
			MinLOD.Default = NewValue;
		}
		else
		{
			int32* ValuePtr = MinLOD.PerQuality.Find(QLKey);
			if (ValuePtr != nullptr)
			{
				*ValuePtr = NewValue;
			}
		}
		StaticMesh->SetQualityLevelMinLOD(MoveTemp(MinLOD));
		StaticMesh->Modify();
		StaticMesh->GetOnMeshChanged().Broadcast();
	}
	StaticMeshEditor.RefreshViewport();
}

void FLevelOfDetailSettingsLayout::OnMinQualityLevelLODCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel)
{
	OnMinQualityLevelLODChanged(InValue, QualityLevel);
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::GetMinQualityLevelLODWidget(FName QualityLevelName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FLevelOfDetailSettingsLayout::GetMinQualityLevelLOD, QualityLevelName)
		.OnValueChanged(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnMinQualityLevelLODChanged, QualityLevelName)
		.OnValueCommitted(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnMinQualityLevelLODCommitted, QualityLevelName)
		.MinValue(0)
		.MaxValue(MAX_STATIC_MESH_LODS)
		.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetMinLODTooltip)
		.IsEnabled(FLevelOfDetailSettingsLayout::GetLODCount() > 1);
}

bool FLevelOfDetailSettingsLayout::AddMinLODQualityLevelOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("AddMinLODQualityLevelOverride", "Add Min LOD Quality Level Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (StaticMesh->GetQualityLevelMinLOD().PerQuality.Find(QLKey) == nullptr)
	{
		FPerQualityLevelInt MinLOD = StaticMesh->GetQualityLevelMinLOD();
		int32 Value = MinLOD.Default;
		MinLOD.PerQuality.Add(QLKey, Value);
		StaticMesh->SetQualityLevelMinLOD(MoveTemp(MinLOD));
		OnMinQualityLevelLODChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

bool FLevelOfDetailSettingsLayout::RemoveMinLODQualityLevelOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveMinLODQualityLevelOverride", "Remove Min LOD Quality Level Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();

	FPerQualityLevelInt MinLOD = StaticMesh->GetQualityLevelMinLOD();
	int32 QL = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (QL != INDEX_NONE && MinLOD.PerQuality.Remove(QL) != 0)
	{
		int32 Value = MinLOD.Default;
		StaticMesh->SetQualityLevelMinLOD(MoveTemp(MinLOD));
		OnMinQualityLevelLODChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

TArray<FName> FLevelOfDetailSettingsLayout::GetMinQualityLevelLODOverrideNames() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	TArray<FName> OverrideNames;
	for (const TPair<int32, int32>& Pair : StaticMesh->GetQualityLevelMinLOD().PerQuality)
	{
		OverrideNames.Add(QualityLevelProperty::QualityLevelToFName(Pair.Key));
	}
	OverrideNames.Sort(FNameLexicalLess());
	return OverrideNames;
}

void FLevelOfDetailSettingsLayout::OnNoRefStreamingLODBiasChanged(int32 NewValue, FName QualityLevel)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	{
		FStaticMeshComponentRecreateRenderStateContext ReregisterContext(StaticMesh, false);
		NewValue = FMath::Clamp<int32>(NewValue, -1, MAX_STATIC_MESH_LODS - 1);
		FPerQualityLevelInt NoRefStreamingLODBias = StaticMesh->GetNoRefStreamingLODBias();
		int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
		if (QualityLevel == NAME_None || QLKey == INDEX_NONE)
		{
			NoRefStreamingLODBias.Default = NewValue;
		}
		else
		{
			int32* ValuePtr = NoRefStreamingLODBias.PerQuality.Find(QLKey);
			if (ValuePtr != nullptr)
			{
				*ValuePtr = NewValue;
			}
		}
		StaticMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		StaticMesh->Modify();
	}
	StaticMeshEditor.RefreshViewport();
}

FReply FLevelOfDetailSettingsLayout::ResetToDefault()
{
	if (GEngine->UseStaticMeshMinLODPerQualityLevels)
	{
		UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
		check(StaticMesh);

		FPerPlatformInt PlatformMinLOD;
		StaticMesh->SetMinLOD(MoveTemp(PlatformMinLOD));
		StaticMesh->Modify();

		StaticMeshEditor.RefreshTool();
	}
	return FReply::Handled();
}

void FLevelOfDetailSettingsLayout::OnNoRefStreamingLODBiasCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel)
{
	OnNoRefStreamingLODBiasChanged(InValue, QualityLevel);
}

int32 FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBias(FName QualityLevel) const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
	const int32* ValuePtr = (QualityLevel == NAME_None) ? nullptr : StaticMesh->GetNoRefStreamingLODBias().PerQuality.Find(QLKey);
	return (ValuePtr != nullptr) ? *ValuePtr : StaticMesh->GetNoRefStreamingLODBias().Default;
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasWidget(FName QualityLevelName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBias, QualityLevelName)
		.OnValueChanged(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnNoRefStreamingLODBiasChanged, QualityLevelName)
		.OnValueCommitted(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnNoRefStreamingLODBiasCommitted, QualityLevelName)
		.MinValue(-1)
		.MaxValue(MAX_STATIC_MESH_LODS - 1)
		.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasTooltip)
		.IsEnabled(FLevelOfDetailSettingsLayout::GetLODCount() > 1);
}

bool FLevelOfDetailSettingsLayout::AddNoRefStreamingLODBiasOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("AddNoRefStreamingLODBiasOverride", "Add NoRef Streaming LOD Bias Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (StaticMesh->GetNoRefStreamingLODBias().PerQuality.Find(QLKey) == nullptr)
	{
		FPerQualityLevelInt NoRefStreamingLODBias = StaticMesh->GetNoRefStreamingLODBias();
		int32 Value = NoRefStreamingLODBias.Default;
		NoRefStreamingLODBias.PerQuality.Add(QLKey, Value);
		StaticMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		OnNoRefStreamingLODBiasChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

bool FLevelOfDetailSettingsLayout::RemoveNoRefStreamingLODBiasOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveNoRefStreamingLODBiasOverride", "Remove NoRef Streaming LOD Bias Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();

	FPerQualityLevelInt NoRefStreamingLODBias = StaticMesh->GetNoRefStreamingLODBias();
	int32 QL = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (QL != INDEX_NONE && NoRefStreamingLODBias.PerQuality.Remove(QL) != 0)
	{
		int32 Value = NoRefStreamingLODBias.Default;
		StaticMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		OnNoRefStreamingLODBiasChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

TArray<FName> FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasOverrideNames() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	TArray<FName> OverrideNames;
	for (const TPair<int32, int32>& Pair : StaticMesh->GetNoRefStreamingLODBias().PerQuality)
	{
		OverrideNames.Add(QualityLevelProperty::QualityLevelToFName(Pair.Key));
	}
	OverrideNames.Sort(FNameLexicalLess());
	return OverrideNames;
}

FText FLevelOfDetailSettingsLayout::GetNoRefStreamingLODBiasTooltip() const
{
	return LOCTEXT("NoRefStreamingLODBiasTooltip", "LOD bias for preloading no-ref mesh LODs. To use platform default, set to -1.");
}

/** @return - whether value was different */
static bool UpdateStaticMeshNumStreamedLODsHelper(UStaticMesh* StaticMesh, int32 NewValue, FName Platform)
{
	bool bWasDifferent = false;
	StaticMesh->Modify();
	{
		FStaticMeshComponentRecreateRenderStateContext ReregisterContext(StaticMesh, false);
		NewValue = FMath::Clamp<int32>(NewValue, -1, MAX_STATIC_MESH_LODS);
		if (Platform == NAME_None)
		{
			bWasDifferent = StaticMesh->GetNumStreamedLODs().Default != NewValue;
			StaticMesh->GetNumStreamedLODs().Default = NewValue;
		}
		else
		{
			int32* ValuePtr = StaticMesh->GetNumStreamedLODs().PerPlatform.Find(Platform);
			if (ValuePtr != nullptr)
			{
				bWasDifferent = *ValuePtr != NewValue;
				*ValuePtr = NewValue;
			}
		}
	}
	return bWasDifferent;
}

void FLevelOfDetailSettingsLayout::OnNumStreamedLODsChanged(int32 NewValue, FName Platform)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	UpdateStaticMeshNumStreamedLODsHelper(StaticMesh, NewValue, Platform);
	StaticMeshEditor.RefreshViewport();
}

void FLevelOfDetailSettingsLayout::OnNumStreamedLODsCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName Platform)
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	if (UpdateStaticMeshNumStreamedLODsHelper(StaticMesh, InValue, Platform))
	{
		if (IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::StaticMesh))
		{
			// Make sure FStaticMeshRenderData::CurrentFirstLODIdx is not accessed on other threads
			IStreamingManager::Get().GetRenderAssetStreamingManager().BlockTillAllRequestsFinished();
		}
		// Recache derived data and relink streaming
		ApplyChanges();
	}
	StaticMeshEditor.RefreshViewport();
}

int32 FLevelOfDetailSettingsLayout::GetNumStreamedLODs(FName Platform) const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	int32* ValuePtr = (Platform == NAME_None) ? nullptr : StaticMesh->GetNumStreamedLODs().PerPlatform.Find(Platform);
	return (ValuePtr != nullptr) ? *ValuePtr : StaticMesh->GetNumStreamedLODs().Default;
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::GetNumStreamedLODsWidget(FName PlatformGroupName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FLevelOfDetailSettingsLayout::GetNumStreamedLODs, PlatformGroupName)
		.OnValueChanged(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnNumStreamedLODsChanged, PlatformGroupName)
		.OnValueCommitted(const_cast<FLevelOfDetailSettingsLayout*>(this), &FLevelOfDetailSettingsLayout::OnNumStreamedLODsCommitted, PlatformGroupName)
		.MinValue(-1)
		.MaxValue(MAX_STATIC_MESH_LODS)
		.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetNumStreamedLODsTooltip)
		.IsEnabled(FLevelOfDetailSettingsLayout::GetLODCount() > 1);
}

bool FLevelOfDetailSettingsLayout::AddNumStreamedLODsPlatformOverride(FName PlatformGroupName)
{
	FScopedTransaction Transaction(LOCTEXT("AddNumStreamedLODsPlatformOverride", "Add NumStreamdLODs Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	if (StaticMesh->GetNumStreamedLODs().PerPlatform.Find(PlatformGroupName) == nullptr)
	{
		int32 Value = StaticMesh->GetNumStreamedLODs().Default;
		StaticMesh->GetNumStreamedLODs().PerPlatform.Add(PlatformGroupName, Value);
		OnNumStreamedLODsChanged(Value, PlatformGroupName);
		return true;
	}
	return false;
}

bool FLevelOfDetailSettingsLayout::RemoveNumStreamedLODsPlatformOverride(FName PlatformGroupName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveNumStreamedLODsPlatformOverride", "Remove NumStreamedLODs Platform Override"));
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	StaticMesh->Modify();
	if (StaticMesh->GetNumStreamedLODs().PerPlatform.Remove(PlatformGroupName) != 0)
	{
		OnNumStreamedLODsChanged(StaticMesh->GetNumStreamedLODs().Default, PlatformGroupName);
		return true;
	}
	return false;
}

TArray<FName> FLevelOfDetailSettingsLayout::GetNumStreamedLODsPlatformOverrideNames() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	TArray<FName> KeyArray;
	StaticMesh->GetNumStreamedLODs().PerPlatform.GenerateKeyArray(KeyArray);
	KeyArray.Sort(FNameLexicalLess());
	return KeyArray;
}

FText FLevelOfDetailSettingsLayout::GetNumStreamedLODsTooltip() const
{
	return LOCTEXT("NumStreamedLODsTooltip", "If non-negative, the number of LODs that can be streamed. Only has effect if mesh LOD streaming is enabled on the target platform.");
}

FText FLevelOfDetailSettingsLayout::GetLODCustomModeNameContent(int32 LODIndex) const
{
	int32 CurrentLodIndex = 0;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		CurrentLodIndex = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
	}
	int32 RealCurrentLODIndex = (CurrentLodIndex == 0 ? 0 : CurrentLodIndex - 1);
	if (LODIndex == INDEX_NONE)
	{
		return LOCTEXT("GetLODCustomModeNameContent", "Custom");
	}
	return FText::Format(LOCTEXT("GetLODModeNameContent", "LOD{0}"), LODIndex);
}

ECheckBoxState FLevelOfDetailSettingsLayout::IsLODCustomModeCheck(int32 LODIndex) const
{
	int32 CurrentLodIndex = 0;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		CurrentLodIndex = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
	}
	if (LODIndex == INDEX_NONE)
	{
		return StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return DetailDisplayLODs[LODIndex] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLevelOfDetailSettingsLayout::SetLODCustomModeCheck(ECheckBoxState NewState, int32 LODIndex)
{
	int32 CurrentLodIndex = 0;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		CurrentLodIndex = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
	}
	if (LODIndex == INDEX_NONE)
	{
		if (NewState == ECheckBoxState::Unchecked)
		{
			StaticMeshEditor.SetCustomData(CustomDataKey_LODEditMode, 0);
			SectionSettingsWidgets[0]->SetCurrentLOD(CurrentLodIndex);
			for (int32 DetailLODIndex = 0; DetailLODIndex < MAX_STATIC_MESH_LODS; ++DetailLODIndex)
			{
				if (!LodCategories.IsValidIndex(DetailLODIndex))
				{
					break;
				}
				LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailLODIndex == (CurrentLodIndex == 0 ? 0 : CurrentLodIndex-1));
			}
		}
		else
		{
			StaticMeshEditor.SetCustomData(CustomDataKey_LODEditMode, 1);
			SectionSettingsWidgets[0]->SetCurrentLOD(0);
		}
	}
	else if(StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		DetailDisplayLODs[LODIndex] = NewState == ECheckBoxState::Checked;
		StaticMeshEditor.SetCustomData(CustomDataKey_LODVisibilityState + LODIndex, DetailDisplayLODs[LODIndex] ? 1 : 0);
	}

	if (StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		for (int32 DetailLODIndex = 0; DetailLODIndex < MAX_STATIC_MESH_LODS; ++DetailLODIndex)
		{
			if (!LodCategories.IsValidIndex(DetailLODIndex))
			{
				break;
			}
			LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailDisplayLODs[DetailLODIndex]);
		}
	}

	if (LodCustomCategory != nullptr)
	{
		LodCustomCategory->SetShowAdvanced(StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0);
	}
}

bool FLevelOfDetailSettingsLayout::IsLODCustomModeEnable(int32 LODIndex) const
{
	if (LODIndex == INDEX_NONE)
	{
		// Custom checkbox is always enable
		return true;
	}
	return StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0;
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::OnGenerateLodComboBoxForLodPicker()
{
	return SNew(SComboButton)
		//.Visibility(this, &FLevelOfDetailSettingsLayout::LodComboBoxVisibilityForLodPicker)
		.IsEnabled(this, &FLevelOfDetailSettingsLayout::IsLodComboBoxEnabledForLodPicker)
		.OnGetMenuContent(this, &FLevelOfDetailSettingsLayout::OnGenerateLodMenuForLodPicker)
		.VAlign(VAlign_Center)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FLevelOfDetailSettingsLayout::GetCurrentLodName)
			.ToolTipText(this, &FLevelOfDetailSettingsLayout::GetCurrentLodTooltip)
		];
}

EVisibility FLevelOfDetailSettingsLayout::LodComboBoxVisibilityForLodPicker() const
{
	//No combo box when in Custom mode
	if (StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		return EVisibility::Hidden;
	}
	return EVisibility::All;
}

bool FLevelOfDetailSettingsLayout::IsLodComboBoxEnabledForLodPicker() const
{
	//No combo box when in Custom mode
	return StaticMeshEditor.GetCustomData(CustomDataKey_LODEditMode) <= 0;
}

TSharedRef<SWidget> FLevelOfDetailSettingsLayout::OnGenerateLodMenuForLodPicker()
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();

	if (StaticMesh == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	bool bAutoLod = false;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		bAutoLod = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel == 0;
	}
	const int32 StaticMeshLODCount = StaticMesh->GetNumLODs();
	if (StaticMeshLODCount < 2)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, NULL);

	FText AutoLodText = FText::FromString((TEXT("LOD Auto")));
	FUIAction AutoLodAction(FExecuteAction::CreateSP(this, &FLevelOfDetailSettingsLayout::OnSelectedLODChanged, 0));
	MenuBuilder.AddMenuEntry(AutoLodText, LOCTEXT("OnGenerateLodMenuForLodPicker_Auto_ToolTip", "With Auto LOD selected, LOD0's properties are visible for editing."), FSlateIcon(), AutoLodAction);
	// Add a menu item for each texture.  Clicking on the texture will display it in the content browser
	for (int32 AllLodIndex = 0; AllLodIndex < StaticMeshLODCount; ++AllLodIndex)
	{
		FText LODLevelString = FText::FromString((TEXT("LOD ") + FString::FromInt(AllLodIndex)));
		FUIAction Action(FExecuteAction::CreateSP(this, &FLevelOfDetailSettingsLayout::OnSelectedLODChanged, AllLodIndex + 1));
		MenuBuilder.AddMenuEntry(LODLevelString, FText::GetEmpty(), FSlateIcon(), Action);
	}

	return MenuBuilder.MakeWidget();
}

void FLevelOfDetailSettingsLayout::OnSelectedLODChanged(int32 NewLodIndex)
{
	if (StaticMeshEditor.GetStaticMeshComponent() == nullptr)
	{
		return;
	}
	int32 CurrentDisplayLOD = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel;
	int32 RealNewLOD = NewLodIndex == 0 ? 0 : NewLodIndex - 1;

	if (CurrentDisplayLOD == NewLodIndex || !LodCategories.IsValidIndex(RealNewLOD))
	{
		return;
	}

	StaticMeshEditor.GetStaticMeshComponent()->SetForcedLodModel(NewLodIndex);

	//Reset the preview section since we do not edit the same LOD
	StaticMeshEditor.GetStaticMeshComponent()->SetSectionPreview(INDEX_NONE);
	StaticMeshEditor.GetStaticMeshComponent()->SelectedEditorSection = INDEX_NONE;

	//Broadcast that the LOD model has changed
	StaticMeshEditor.BroadcastOnSelectedLODChanged();
}

FText FLevelOfDetailSettingsLayout::GetCurrentLodName() const
{
	bool bAutoLod = false;
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr)
	{
		bAutoLod = StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel == 0;
	}
	int32 CurrentDisplayLOD = bAutoLod ? 0 : StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel - 1;
	return FText::FromString(bAutoLod ? FString(TEXT("LOD Auto")) : (FString(TEXT("LOD ")) + FString::FromInt(CurrentDisplayLOD)));
}

FText FLevelOfDetailSettingsLayout::GetCurrentLodTooltip() const
{
	if (StaticMeshEditor.GetStaticMeshComponent() != nullptr && StaticMeshEditor.GetStaticMeshComponent()->ForcedLodModel == 0)
	{
		return LOCTEXT("StaticMeshEditorLODPickerCurrentLODTooltip", "With Auto LOD selected, LOD0's properties are visible for editing");
	}
	return FText::GetEmpty();
}

bool FLevelOfDetailSettingsLayout::IsNaniteEnabled() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh != nullptr);
	return StaticMesh->IsNaniteEnabled();
}

/////////////////////////////////
// FRayTracingProxySettingsLayout
/////////////////////////////////

FRayTracingProxySettingsLayout::FRayTracingProxySettingsLayout(FStaticMeshEditor& InStaticMeshEditor)
	: StaticMeshEditor(InStaticMeshEditor)
{
	const UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	RayTracingProxySettings = StaticMesh->GetRayTracingProxySettings();
}

FRayTracingProxySettingsLayout::~FRayTracingProxySettingsLayout()
{
}

const FMeshRayTracingProxySettings& FRayTracingProxySettingsLayout::GetSettings() const
{
	return RayTracingProxySettings;
}

void FRayTracingProxySettingsLayout::UpdateSettings(const FMeshRayTracingProxySettings& InSettings)
{
	checkf(!ShouldGenerateRayTracingProxiesByDefault() || RayTracingProxySettings.bEnabled == InSettings.bEnabled,
		TEXT("RayTracingProxySettings.bEnabled shouldn't be modified when ray tracing proxies are enabled by default."));

	RayTracingProxySettings = InSettings;
}

bool FRayTracingProxySettingsLayout::IsApplyNeeded() const
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);
	return StaticMesh->GetRayTracingProxySettings() != RayTracingProxySettings;
}

void FRayTracingProxySettingsLayout::ApplyChanges()
{
	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	check(StaticMesh);

	checkf(!ShouldGenerateRayTracingProxiesByDefault() || StaticMesh->GetRayTracingProxySettings().bEnabled == RayTracingProxySettings.bEnabled,
		TEXT("RayTracingProxySettings.bEnabled shouldn't be modified when ray tracing proxies are enabled by default."));

	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("StaticMeshName"), FText::FromString(StaticMesh->GetName()));
		FScopedSlowTask SlowTask(0, FText::Format(LOCTEXT("ApplyRayTracingChanges", "Applying changes to {StaticMeshName}..."), Args), true);
		SlowTask.MakeDialog();

		StaticMesh->Modify();
		StaticMesh->SetRayTracingProxySettings(RayTracingProxySettings);

		StaticMesh->NotifyRayTracingProxySettingsChanged();
	}


	StaticMeshEditor.RefreshTool();
}

FReply FRayTracingProxySettingsLayout::OnApply()
{
	ApplyChanges();
	return FReply::Handled();
}

ECheckBoxState FRayTracingProxySettingsLayout::IsEnabledChecked() const
{
	if (ShouldGenerateRayTracingProxiesByDefault())
	{
		return ShouldEnable() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	return RayTracingProxySettings.bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FRayTracingProxySettingsLayout::OnEnabledChanged(ECheckBoxState NewState)
{
	checkf(!ShouldGenerateRayTracingProxiesByDefault(), TEXT("RayTracingProxySettings.bEnabled shouldn't be modified when ray tracing proxies are enabled by default."));
	RayTracingProxySettings.bEnabled = NewState == ECheckBoxState::Checked ? true : false;
}

float FRayTracingProxySettingsLayout::GetFallbackPercentTriangles() const
{
	return RayTracingProxySettings.FallbackPercentTriangles * 100.0f; // Display fraction as percentage.
}

void FRayTracingProxySettingsLayout::OnFallbackPercentTrianglesChanged(float NewValue)
{
	RayTracingProxySettings.FallbackPercentTriangles = NewValue * 0.01f;
}

void FRayTracingProxySettingsLayout::OnFallbackPercentTrianglesCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.RayTracingProxySettings"), TEXT("FallbackPercentTriangles"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnFallbackPercentTrianglesChanged(NewValue);
}

float FRayTracingProxySettingsLayout::GetFallbackRelativeError() const
{
	return RayTracingProxySettings.FallbackRelativeError;
}

void FRayTracingProxySettingsLayout::OnFallbackRelativeErrorChanged(float NewValue)
{
	RayTracingProxySettings.FallbackRelativeError = NewValue;
}

void FRayTracingProxySettingsLayout::OnFallbackRelativeErrorCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.RayTracingProxySettings"), TEXT("FallbackRelativeError"), FString::Printf(TEXT("%.1f"), NewValue));
	}
	OnFallbackRelativeErrorChanged(NewValue);
}

float FRayTracingProxySettingsLayout::GetFoliageOverOcclusionBias() const
{
	return RayTracingProxySettings.FoliageOverOcclusionBias;
}

void FRayTracingProxySettingsLayout::OnFoliageOverOcclusionBiasChanged(float NewValue)
{
	RayTracingProxySettings.FoliageOverOcclusionBias = NewValue;
}

void FRayTracingProxySettingsLayout::OnFoliageOverOcclusionBiasCommitted(float NewValue, ETextCommit::Type TextCommitType)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.RayTracingProxySettings"), TEXT("FoliageOverOcclusionBias"), FString::Printf(TEXT("%.2f"), NewValue));
	}
	OnFoliageOverOcclusionBiasChanged(NewValue);
}

void FRayTracingProxySettingsLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("RayTracingProxySettings", "Ray Tracing Proxy Settings"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

bool FRayTracingProxySettingsLayout::ShouldEnable() const
{
	const UStaticMesh* StaticMesh = StaticMeshEditor.GetStaticMesh();
	return StaticMesh && StaticMesh->bSupportRayTracing && StaticMesh->GetNaniteSettings().bEnabled;
}

bool FRayTracingProxySettingsLayout::CanToggleEnabled() const
{
	// can only toggle ray tracing proxy checkbox is the project setting is not enabling it by default
	return ShouldEnable() && !ShouldGenerateRayTracingProxiesByDefault();
}


void FRayTracingProxySettingsLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	TAttribute<bool> CanToggleEnabledAttr = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateRaw(this, &FRayTracingProxySettingsLayout::CanToggleEnabled));

	TSharedPtr<SCheckBox> RayTracingProxyEnabledCheck;
	{
		FText EnabledRayTracingSupportTooltipText = LOCTEXT("EnabledRayTracingSupportTooltip", "Use dedicated ray tracing proxy mesh instead of reusing the Nanite Fallback Mesh.");

		if (!ShouldEnable())
		{
			const FText ExtraTooltipText = LOCTEXT("EnabledRayTracingSupportNaniteOnlyTooltip", "(Currently only supported for Nanite meshes).");
			EnabledRayTracingSupportTooltipText = FText::Format(FText::FromString("{0}\n{1}"), EnabledRayTracingSupportTooltipText, ExtraTooltipText);
		}
		else if (ShouldGenerateRayTracingProxiesByDefault())
		{
			const FText ExtraTooltipText = LOCTEXT("EnabledRayTracingSupportProjectOverrideTooltip", "(This setting is being overriden by the project setting \"Generate Ray Tracing Proxies\").");
			EnabledRayTracingSupportTooltipText = FText::Format(FText::FromString("{0}\n{1}"), EnabledRayTracingSupportTooltipText, ExtraTooltipText);
		}

		ChildrenBuilder.AddCustomRow(LOCTEXT("Enabled", "Enabled"))
		.RowTag("EnabledRayTracingSupport")
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("EnabledRayTracingSupport", "Enable Ray Tracing Proxy Support"))
			.ToolTipText(EnabledRayTracingSupportTooltipText)
		]
		.ValueContent()
		[
			SAssignNew(RayTracingProxyEnabledCheck, SCheckBox)
			.IsChecked(this, &FRayTracingProxySettingsLayout::IsEnabledChecked)
			.OnCheckStateChanged(this, &FRayTracingProxySettingsLayout::OnEnabledChanged)
		]
		.IsEnabled(CanToggleEnabledAttr);
	}

	TAttribute<bool> RayTracingProxyEnabledAttr = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(
		[this, RayTracingProxyEnabledCheck]()
		{
			return ShouldEnable() && RayTracingProxyEnabledCheck->IsChecked();
		}));

	{
		TSharedRef<FStructOnScope> TempStruct = MakeShared<FStructOnScope>(FMeshRayTracingProxySettings::StaticStruct());
		FMeshRayTracingProxySettings::StaticStruct()->CopyScriptStruct(TempStruct->GetStructMemory(), &RayTracingProxySettings, 1);
		IDetailPropertyRow* PropertyRow = ChildrenBuilder.AddExternalStructureProperty(TempStruct, GET_MEMBER_NAME_CHECKED(FMeshRayTracingProxySettings, FallbackTarget));
		PropertyRow->GetPropertyHandle()->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
			[this, TempStruct]
			{
				FMeshRayTracingProxySettings* TempStruct2 = (FMeshRayTracingProxySettings*)TempStruct->GetStructMemory();
				RayTracingProxySettings.FallbackTarget = TempStruct2->FallbackTarget;
			}
		));
		PropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda(
			[this, TempStruct]
			{
				FMeshRayTracingProxySettings* TempStruct2 = (FMeshRayTracingProxySettings*)TempStruct->GetStructMemory();
				RayTracingProxySettings.FallbackTarget = TempStruct2->FallbackTarget;
			}
		));
		PropertyRow->IsEnabled(RayTracingProxyEnabledAttr);
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("FallbackTrianglePercent", "Fallback Triangle Percent"))
		.RowTag("FallbackTrianglePercent")
		.IsEnabled(RayTracingProxyEnabledAttr)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("FallbackTrianglePercent", "Fallback Triangle Percent"))
			.ToolTipText(LOCTEXT("FallbackTrianglePercentTooltip", "Reduce until no more than this percentage of triangles remain when generating a fallback\nmesh that will be used anywhere the full detail Nanite data can't,\nincluding platforms that don't support Nanite rendering."))
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SSpinBox<float>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0.0f)
			.MaxValue(100.0f)
			.Value(this, &FRayTracingProxySettingsLayout::GetFallbackPercentTriangles)
			.OnValueChanged(this, &FRayTracingProxySettingsLayout::OnFallbackPercentTrianglesChanged)
			.OnValueCommitted(this, &FRayTracingProxySettingsLayout::OnFallbackPercentTrianglesCommitted)
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]()
		{
			return RayTracingProxySettings.FallbackTarget == ENaniteFallbackTarget::PercentTriangles ? EVisibility::Visible : EVisibility::Hidden;
		})));
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("FallbackRelativeError", "Fallback Relative Error"))
		.RowTag("FallbackRelativeError")
		.IsEnabled(RayTracingProxyEnabledAttr)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("FallbackRelativeError", "Fallback Relative Error"))
			.ToolTipText(LOCTEXT("FallbackRelativeErrorTooltip", "Reduce until at least this amount of error is reached relative to its size\nwhen generating a fallback mesh that will be used anywhere the full detail Nanite data can't,\nincluding platforms that don't support Nanite rendering."))
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SSpinBox<float>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0.0f)
			.Value(this, &FRayTracingProxySettingsLayout::GetFallbackRelativeError)
			.OnValueChanged(this, &FRayTracingProxySettingsLayout::OnFallbackRelativeErrorChanged)
			.OnValueCommitted(this, &FRayTracingProxySettingsLayout::OnFallbackRelativeErrorCommitted)
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]()
		{
			return RayTracingProxySettings.FallbackTarget == ENaniteFallbackTarget::RelativeError ? EVisibility::Visible : EVisibility::Hidden;
		})));
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("FoliageOverOcclusionBias", "Foliage OverOcclusion Bias"))
		.RowTag("FoliageOverOcclusionBias")
		.IsEnabled(RayTracingProxyEnabledAttr)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("FoliageOverOcclusionBias", "Foliage OverOcclusion Bias"))
			.ToolTipText(LOCTEXT("FoliageOverOcclusionBiasTooltip", "A bias to reduce foliage over occlusion in Lumen GI. 0: no adjustment, 1: full strength."))
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SSpinBox<float>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0.0f)
			.MaxValue(0.9f)
			.Value(this, &FRayTracingProxySettingsLayout::GetFoliageOverOcclusionBias)
			.OnValueChanged(this, &FRayTracingProxySettingsLayout::OnFoliageOverOcclusionBiasChanged)
			.OnValueCommitted(this, &FRayTracingProxySettingsLayout::OnFoliageOverOcclusionBiasCommitted)
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("ApplyChanges", "Apply Changes"))
		.RowTag("ApplyChanges")
		.ValueContent()
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.OnClicked(this, &FRayTracingProxySettingsLayout::OnApply)
			.IsEnabled(this, &FRayTracingProxySettingsLayout::IsApplyNeeded)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ApplyChanges", "Apply Changes"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE
