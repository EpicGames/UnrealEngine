// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Engine/EngineTypes.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "MeshReductionSettings.h"
#include "Engine/StaticMesh.h"
#include "IDetailCustomization.h"
#include "StaticMeshResources.h"
#include "Widgets/Input/SSpinBox.h"
#include "IDetailCustomNodeBuilder.h"

struct FAssetData;
class FAssetThumbnailPool;
class FDetailWidgetRow;
class FLevelOfDetailSettingsLayout;
class FNaniteStaticMeshLayout;
class FRayTracingProxySettingsLayout;
class FStaticMeshEditor;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IDetailGroup;
class IStaticMeshEditor;
class UMaterialInterface;
struct FSectionLocalizer;

enum ECreationModeChoice
{
	CreateNew,
	UseChannel0,
};

enum ELimitModeChoice
{
	Stretching,
	Charts
};

class FStaticMeshDetails : public IDetailCustomization
{
public:
	FStaticMeshDetails( class FStaticMeshEditor& InStaticMeshEditor );
	~FStaticMeshDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( class IDetailLayoutBuilder& DetailBuilder ) override;

	/** @return true if settings have been changed and need to be applied to the static mesh */
	bool IsApplyNeeded() const;

	/** Applies changes to the static mesh */
	void ApplyChanges();

private:
	/** Level of detail settings for the details panel */
	TSharedPtr<FLevelOfDetailSettingsLayout> LevelOfDetailSettings;

	/** Nanite settings for the details panel. */
	TSharedPtr<FNaniteStaticMeshLayout> NaniteSettings;

	/** Nanite settings for the details panel. */
	TSharedPtr<FRayTracingProxySettingsLayout> RayTracingProxySettings;

	/** Static mesh editor */
	class FStaticMeshEditor& StaticMeshEditor;

	// Property handle used to determine if the VertexColorImportOverride property should be enabled.
	TSharedPtr<IPropertyHandle> VertexColorImportOptionHandle;
	
	// Property handle used during UI construction
	TSharedPtr<IPropertyHandle> VertexColorImportOverrideHandle;

	// Delegate implementation of FOnInstancedPropertyIteration used during DataImport UI construction
	void OnInstancedFbxStaticMeshImportDataPropertyIteration(IDetailCategoryBuilder& BaseCategory, IDetailGroup* PropertyGroup, TSharedRef<IPropertyHandle>& Property) const;

	// Delegate to ensure the lightmap settings are always valid.
	void OnLightmapSettingsChanged();

	// Delegate used at runtime to determine the state of the VertexOverrideColor property
	bool GetVertexOverrideColorEnabledState() const;
};


/** 
 * Window handles convex decomposition, settings and controls.
 */
class SConvexDecomposition : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SConvexDecomposition ) :
		_StaticMeshEditorPtr()
		{
		}
		/** The Static Mesh Editor this tool is associated with. */
		SLATE_ARGUMENT( TWeakPtr< IStaticMeshEditor >, StaticMeshEditorPtr )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual ~SConvexDecomposition();

private:
	/** Callback when the "Apply" button is clicked. */
	FReply OnApplyDecomp();

	/** Callback when the "Defaults" button is clicked. */
	FReply OnDefaults();

	/** Assigns the accuracy of hulls based on the spinbox's value. */
	void OnHullCountCommitted(uint32 InNewValue, ETextCommit::Type CommitInfo);

	/** Assigns the accuracy of hulls based on the spinbox's value. */
	void OnHullCountChanged(uint32 InNewValue);

	/** Retrieves the precision of hulls created. */
	uint32 GetHullPrecision() const;

	/** Assigns the precision of hulls based on the spinbox's value. */
	void OnHullPrecisionCommitted(uint32 InNewValue, ETextCommit::Type CommitInfo);

	/** Assigns the precision of hulls based on the spinbox's value. */
	void OnHullPrecisionChanged(uint32 InNewValue);

	/** Retrieves the accuracy of hulls created. */
	uint32 GetHullCount() const;

	/** Assigns the max number of hulls based on the spinbox's value. */
	void OnVertsPerHullCountCommitted(int32 InNewValue, ETextCommit::Type CommitInfo);

	/** Assigns the max number of hulls based on the spinbox's value. */
	void OnVertsPerHullCountChanged(int32 InNewValue);

	/** 
	 *	Retrieves the max number of verts per hull allowed.
	 *
	 *	@return			The max number of verts per hull selected.
	 */
	int32 GetVertsPerHullCount() const;

private:
	/** The Static Mesh Editor this tool is associated with. */
	TWeakPtr<IStaticMeshEditor> StaticMeshEditorPtr;

	/** Spinbox for the max number of hulls allowed. */
	TSharedPtr< SSpinBox<uint32> > HullCount;

	/** Spinbox for the convex decomposition precision allowed. */
	TSharedPtr< SSpinBox<uint32> > HullPrecision;

	/** The current number of max number of hulls selected. */
	uint32 CurrentHullCount;

	/** The current precision level for convex decomposition */
	uint32 CurrentHullPrecision;

	/** Spinbox for the max number of verts per hulls allowed. */
	TSharedPtr< SSpinBox<int32> > MaxVertsPerHull;

	/** The current number of max verts per hull selected. */
	int32 CurrentMaxVertsPerHullCount;

};


class FMeshBuildSettingsLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FMeshBuildSettingsLayout>
{
public:
	FMeshBuildSettingsLayout( TSharedRef<FLevelOfDetailSettingsLayout> InParentLODSettings, int32 InLODIndex );
	virtual ~FMeshBuildSettingsLayout();

	const FMeshBuildSettings& GetSettings() const { return BuildSettings; }
	void UpdateSettings(const FMeshBuildSettings& InSettings);

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {}
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override{}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName MeshBuildSettings("MeshBuildSettings"); return MeshBuildSettings; }
	virtual bool InitiallyCollapsed() const override { return true; }

	FReply OnApplyChanges();
	ECheckBoxState ShouldRecomputeNormals() const;
	ECheckBoxState ShouldRecomputeTangents() const;
	ECheckBoxState ShouldUseMikkTSpace() const;
	ECheckBoxState ShouldComputeWeightedNormals() const;
	ECheckBoxState ShouldRemoveDegenerates() const;
	ECheckBoxState ShouldBuildReversedIndexBuffer() const;
	ECheckBoxState ShouldUseHighPrecisionTangentBasis() const;
	ECheckBoxState ShouldUseFullPrecisionUVs() const;
	ECheckBoxState ShouldUseBackwardsCompatibleF16TruncUVs() const;
	ECheckBoxState ShouldGenerateLightmapUVs() const;
	ECheckBoxState ShouldGenerateDistanceFieldAsIfTwoSided() const;
	bool IsRemoveDegeneratesDisabled() const;
	int32 GetMinLightmapResolution() const;
	int32 GetSrcLightmapIndex() const;
	int32 GetDstLightmapIndex() const;
	TOptional<float> GetBuildScaleX() const;
	TOptional<float> GetBuildScaleY() const;
	TOptional<float> GetBuildScaleZ() const;
	float GetDistanceFieldResolutionScale() const;
	int32 GetMaxLumenMeshCards() const;

	void OnRecomputeNormalsChanged(ECheckBoxState NewState);
	void OnRecomputeTangentsChanged(ECheckBoxState NewState);
	void OnUseMikkTSpaceChanged(ECheckBoxState NewState);
	void OnComputeWeightedNormalsChanged(ECheckBoxState NewState);
	void OnRemoveDegeneratesChanged(ECheckBoxState NewState);
	void OnBuildReversedIndexBufferChanged(ECheckBoxState NewState);
	void OnUseHighPrecisionTangentBasisChanged(ECheckBoxState NewState);
	void OnUseFullPrecisionUVsChanged(ECheckBoxState NewState);
	void OnUseBackwardsCompatibleF16TruncUVsChanged(ECheckBoxState NewState);
	void OnGenerateLightmapUVsChanged(ECheckBoxState NewState);
	void OnGenerateDistanceFieldAsIfTwoSidedChanged(ECheckBoxState NewState);
	void OnMinLightmapResolutionChanged( int32 NewValue );
	void OnSrcLightmapIndexChanged( int32 NewValue );
	void OnDstLightmapIndexChanged( int32 NewValue );
	void OnBuildScaleXChanged( float NewScaleX, ETextCommit::Type TextCommitType );
	void OnBuildScaleYChanged( float NewScaleY, ETextCommit::Type TextCommitType );
	void OnBuildScaleZChanged( float NewScaleZ, ETextCommit::Type TextCommitType );
	void OnDistanceFieldResolutionScaleChanged(float NewValue);
	void OnDistanceFieldResolutionScaleCommitted(float NewValue, ETextCommit::Type TextCommitType);
	FString GetCurrentDistanceFieldReplacementMeshPath() const;
	void OnDistanceFieldReplacementMeshSelected(const FAssetData& AssetData);
	void OnMaxLumenMeshCardsChanged(int32 NewValue);
	void OnMaxLumenMeshCardsCommitted(int32 NewValue, ETextCommit::Type TextCommitType);

private:
	TWeakPtr<FLevelOfDetailSettingsLayout> ParentLODSettings;
	FMeshBuildSettings BuildSettings;
	int32 LODIndex;
};

class FMeshReductionSettingsLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FMeshReductionSettingsLayout>
{
public:
	FMeshReductionSettingsLayout(TSharedRef<FLevelOfDetailSettingsLayout> InParentLODSettings, int32 InCurrentLODIndex, bool InCanReduceMyself);
	virtual ~FMeshReductionSettingsLayout();

	const FMeshReductionSettings& GetSettings() const;
	void UpdateSettings(const FMeshReductionSettings& InSettings);
private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {}
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override{}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName MeshReductionSettings("MeshReductionSettings"); return MeshReductionSettings; }
	virtual bool InitiallyCollapsed() const override { return true; }

	FReply OnApplyChanges();

	// used by native tool and simplygon
	float GetPercentTriangles() const;
	uint32 GetMaxNumOfPercentTriangles() const;

	// used by native quadric simplifier
	float GetPercentVertices() const;
	uint32 GetMaxNumOfPercentVertices() const;

	// used by simplygon only
	float GetMaxDeviation() const;
	float GetPixelError() const;
	float GetWeldingThreshold() const;
	ECheckBoxState ShouldRecalculateNormals() const;
	float GetHardAngleThreshold() const;

	// used by native tool and simplygon
	void OnPercentTrianglesChanged(float NewValue);
	void OnPercentTrianglesCommitted(float NewValue, ETextCommit::Type TextCommitType);
	void OnMaxNumOfPercentTrianglesChanged(uint32 NewValue);
	void OnMaxNumOfPercentTrianglesCommitted(uint32 NewValue, ETextCommit::Type TextCommitType);

	// Used by native code only
	void OnPercentVerticesChanged(float NewValue);
	void OnPercentVerticesCommitted(float NewValue, ETextCommit::Type TextCommitType);
	void OnMaxNumOfPercentVerticesChanged(uint32 NewValue);
	void OnMaxNumOfPercentVerticesCommitted(uint32 NewValue, ETextCommit::Type TextCommitType);
	

	//used by simplygon only
	void OnMaxDeviationChanged(float NewValue);
	void OnMaxDeviationCommitted(float NewValue, ETextCommit::Type TextCommitType);
	void OnPixelErrorChanged(float NewValue);
	void OnPixelErrorCommitted(float NewValue, ETextCommit::Type TextCommitType);
	void OnRecalculateNormalsChanged(ECheckBoxState NewValue);

	// used by native tool and simplygon
	void OnWeldingThresholdChanged(float NewValue);
	void OnWeldingThresholdCommitted(float NewValue, ETextCommit::Type TextCommitType);

	// used by simplygon only
	void OnHardAngleThresholdChanged(float NewValue);
	void OnHardAngleThresholdCommitted(float NewValue, ETextCommit::Type TextCommitType);

	void OnSilhouetteImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	void OnTextureImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	void OnShadingImportanceChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	// Used by native tool only.
	void OnTerminationCriterionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	// Are we using our tool, or simplygon?  The tool is only changed during editor restarts
	bool UseNativeToolLayout() const;

	EVisibility GetTriangleCriterionVisibility() const;
	EVisibility GetVertexCriterionVisibility() const;

	TOptional<int32> GetBaseLODIndex() const;
	void SetBaseLODIndex(int32 NewLODBaseIndex);

private:
	TWeakPtr<FLevelOfDetailSettingsLayout> ParentLODSettings;
	FMeshReductionSettings ReductionSettings;
	int32 CurrentLODIndex;
	bool bCanReduceMyself;

	// Used by simplygon
	TArray<TSharedPtr<FString> > ImportanceOptions;
	TSharedPtr<STextComboBox> SilhouetteCombo;
	TSharedPtr<STextComboBox> TextureCombo;
	TSharedPtr<STextComboBox> ShadingCombo;

	// Used by quadric simplifier
	TSharedPtr<STextComboBox> TerminationCriterionCombo;
	TArray<TSharedPtr<FString> > TerminationOptions;

    // Identify the actual that this UI drives
	bool bUseQuadricSimplifier;
};

class FMeshSectionSettingsLayout : public TSharedFromThis<FMeshSectionSettingsLayout>
{
public:
	FMeshSectionSettingsLayout( IStaticMeshEditor& InStaticMeshEditor, int32 InLODIndex, TArray<class IDetailCategoryBuilder*> &InLodCategories)
		: StaticMeshEditor( InStaticMeshEditor )
		, LODIndex( InLODIndex )
		, LodCategoriesPtr(&InLodCategories)
	{}

	virtual ~FMeshSectionSettingsLayout();

	void AddToCategory( IDetailCategoryBuilder& CategoryBuilder );

	void SetCurrentLOD(int32 NewLodIndex);

private:
	
	UStaticMesh& GetStaticMesh() const;

	void OnCopySectionList(int32 CurrentLODIndex);
	bool OnCanCopySectionList(int32 CurrentLODIndex) const;
	void OnPasteSectionList(int32 CurrentLODIndex);

	void OnCopySectionItem(int32 CurrentLODIndex, int32 SectionIndex);
	bool OnCanCopySectionItem(int32 CurrentLODIndex, int32 SectionIndex) const;
	void OnPasteSectionItem(int32 CurrentLODIndex, int32 SectionIndex);

	/**
	* Called by the material list widget when we need to get new materials for the list
	*
	* @param OutMaterials	Handle to a material list builder that materials should be added to
	*/
	void OnGetSectionsForView(class ISectionListBuilder& OutSections, int32 ForLODIndex);

	/**
	* Called when a user drags a new material over a list item to replace it
	*
	* @param NewMaterial	The material that should replace the existing material
	* @param PrevMaterial	The material that should be replaced
	* @param SlotIndex		The index of the slot on the component where materials should be replaces
	* @param bReplaceAll	If true all materials in the slot should be replaced not just ones using PrevMaterial
	*/
	void OnSectionChanged(int32 ForLODIndex, int32 SectionIndex, int32 NewMaterialSlotIndex, FName NewMaterialSlotName);
	
	/**
	* Called by the material list widget on generating each name widget
	*
	* @param Material		The material that is being displayed
	* @param SlotIndex		The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomNameWidgetsForSection(int32 ForLODIndex, int32 SectionIndex);

	/**
	* Called by the material list widget on generating each thumbnail widget
	*
	* @param Material		The material that is being displayed
	* @param SlotIndex		The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomSectionWidgetsForSection(int32 ForLODIndex, int32 SectionIndex);

	ECheckBoxState IsSectionVisibleInRayTracing(int32 SectionIndex) const;
	void OnSectionVisibleInRayTracingChanged(ECheckBoxState NewState, int32 SectionIndex);

	ECheckBoxState DoesSectionAffectDistanceFieldLighting(int32 SectionIndex) const;
	void OnSectionAffectDistanceFieldLightingChanged(ECheckBoxState NewState, int32 SectionIndex);

	ECheckBoxState IsSectionOpaque(int32 SectionIndex) const;
	void OnSectionForceOpaqueFlagChanged(ECheckBoxState NewState, int32 SectionIndex);
	
	ECheckBoxState DoesSectionCastShadow(int32 SectionIndex) const;
	void OnSectionCastShadowChanged(ECheckBoxState NewState, int32 SectionIndex);
	ECheckBoxState DoesSectionCollide(int32 SectionIndex) const;
	bool SectionCollisionEnabled() const;
	FText GetCollisionEnabledToolTip() const;
	void OnSectionCollisionChanged(ECheckBoxState NewState, int32 SectionIndex);

	ECheckBoxState IsSectionHighlighted(int32 SectionIndex) const;
	void OnSectionHighlightedChanged(ECheckBoxState NewState, int32 SectionIndex);
	ECheckBoxState IsSectionIsolatedEnabled(int32 SectionIndex) const;
	void OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex);

	void CallPostEditChange(FProperty* PropertyChanged=nullptr);
	void UpdateLODCategoryVisibility();

	IStaticMeshEditor& StaticMeshEditor;
	int32 LODIndex;

	TArray<class IDetailCategoryBuilder*> *LodCategoriesPtr;
};

struct FSectionLocalizer
{
	FSectionLocalizer(int32 InLODIndex, int32 InSectionIndex)
		: LODIndex(InLODIndex)
		, SectionIndex(InSectionIndex)
	{}

	bool operator==(const FSectionLocalizer& Other) const
	{
		return (LODIndex == Other.LODIndex && SectionIndex == Other.SectionIndex);
	}

	bool operator!=(const FSectionLocalizer& Other) const
	{
		return !((*this) == Other);
	}

	int32 LODIndex;
	int32 SectionIndex;
};


class FMeshMaterialsLayout : public TSharedFromThis<FMeshMaterialsLayout>
{
public:
	FMeshMaterialsLayout(IStaticMeshEditor& InStaticMeshEditor)
		: StaticMeshEditor(InStaticMeshEditor)
	{
		bDeleteWarningConsumed = false;
	}

	virtual ~FMeshMaterialsLayout();

	void AddToCategory(IDetailCategoryBuilder& CategoryBuilder, const TArray<FAssetData>& AssetDataArray);

private:
	UStaticMesh& GetStaticMesh() const;
	FReply AddMaterialSlot();
	FText GetMaterialArrayText() const;

	void GetMaterials(class IMaterialListBuilder& ListBuilder);
	void OnMaterialChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll);
	TSharedRef<SWidget> OnGenerateWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex);
	TSharedRef<SWidget> OnGenerateNameWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex);
	
	/**
	* Called by the material list widget on generating each thumbnail widget
	* Those generated widget will be stack under the material list row
	*
	* @param Material		The material that is being displayed
	* @param SlotIndex		The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateMaterialListExtraBottomWidget(UMaterialInterface* Material, int32 SlotIndex);
	
	/* Call when user change the material slot overlay material. */
	void OnMaterialSlotOverlayMaterialChanged(UMaterialInterface* NewOverlayMaterial, int32 MaterialIndex);

	void OnResetMaterialToDefaultClicked(UMaterialInterface* Material, int32 SlotIndex);

	ECheckBoxState IsMaterialHighlighted(int32 SlotIndex) const;
	void OnMaterialHighlightedChanged(ECheckBoxState NewState, int32 SlotIndex);
	ECheckBoxState IsMaterialIsolatedEnabled(int32 SlotIndex) const;
	void OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 SlotIndex);

	FText GetOriginalImportMaterialNameText(int32 MaterialIndex) const;
	FText GetMaterialNameText(int32 MaterialIndex) const;
	void OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex);
	bool CanDeleteMaterialSlot(int32 MaterialIndex) const;
	void OnDeleteMaterialSlot(int32 MaterialIndex);
	TSharedRef<SWidget> OnGetMaterialSlotUsedByMenuContent(int32 MaterialIndex);
	FText GetFirstMaterialSlotUsedBySection(int32 MaterialIndex) const;

	/* If the material list is dirty this function will return true */
	bool OnMaterialListDirty();

	ECheckBoxState IsShadowCastingEnabled(int32 SlotIndex) const;
	void OnShadowCastingChanged(ECheckBoxState NewState, int32 SlotIndex);

	EVisibility GetOverrideUVDensityVisibililty() const;
	ECheckBoxState IsUVDensityOverridden(int32 SlotIndex) const;
	void OnOverrideUVDensityChanged(ECheckBoxState NewState, int32 SlotIndex);

	EVisibility GetUVDensityVisibility(int32 SlotIndex, int32 UVChannelIndex) const;
	TOptional<float> GetUVDensityValue(int32 SlotIndex, int32 UVChannelIndex) const;
	void SetUVDensityValue(float InDensity, ETextCommit::Type CommitType, int32 SlotIndex, int32 UVChannelIndex);

	SVerticalBox::FSlot& GetUVDensitySlot(int32 SlotIndex, int32 UVChannelIndex) const;

	void CallPostEditChange(FProperty* PropertyChanged = nullptr);

	void OnCopyMaterialList();
	bool OnCanCopyMaterialList() const;
	void OnPasteMaterialList();

	void OnCopyMaterialItem(int32 CurrentSlot);
	bool OnCanCopyMaterialItem(int32 CurrentSlot) const;
	void OnPasteMaterialItem(int32 CurrentSlot);

	IStaticMeshEditor& StaticMeshEditor;
	
	/* This is to know if material are used by any LODs sections. */
	TMap<int32, TArray<FSectionLocalizer>> MaterialUsedMap;

	/*
	* This prevent showing the delete material slot warning dialog more then once per editor session
	*/
	bool bDeleteWarningConsumed;
};

/** 
 * Window for adding and removing LOD.
 */
class FLevelOfDetailSettingsLayout : public TSharedFromThis<FLevelOfDetailSettingsLayout>
{
public:
	FLevelOfDetailSettingsLayout( FStaticMeshEditor& StaticMeshEditor );
	virtual ~FLevelOfDetailSettingsLayout();
	
	void AddToDetailsPanel( IDetailLayoutBuilder& DetailBuilder );

	/** Returns true if settings have been changed and an Apply is needed to update the asset. */
	bool IsApplyNeeded() const;

	/** Apply current LOD settings to the mesh. */
	void ApplyChanges();

	/** Returns true if the LOD's static mesh has Nanite enabled */
	bool IsNaniteEnabled() const;

private:

	/** Creates the UI for Current LOD panel */
	void AddLODLevelCategories( class IDetailLayoutBuilder& DetailBuilder );

	/** Callbacks. */
	FReply OnApply();
	void OnLODCountChanged(int32 NewValue);
	void OnLODCountCommitted(int32 InValue, ETextCommit::Type CommitInfo);
	int32 GetLODCount() const;

	void OnSelectedLODChanged(int32 NewLodIndex);

	void OnMinLODChanged(int32 NewValue, FName Platform);
	void OnMinLODCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName Platform);
	int32 GetMinLOD(FName Platform) const;
	FPerPlatformInt GetMinLOD() const;
	TSharedRef<SWidget> GetMinLODWidget(FName PlatformGroupName) const;
	bool AddMinLODPlatformOverride(FName PlatformGroupName);
	bool RemoveMinLODPlatformOverride(FName PlatformGroupName);
	TArray<FName> GetMinLODPlatformOverrideNames() const;

	void OnMinQualityLevelLODChanged(int32 NewValue, FName QualityLevel);
	void OnMinQualityLevelLODCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel);
	int32 GetMinQualityLevelLOD(FName QualityLevel) const;
	TSharedRef<SWidget> GetMinQualityLevelLODWidget(FName QualityLevelName) const;
	bool AddMinLODQualityLevelOverride(FName QualityLevelName);
	bool RemoveMinLODQualityLevelOverride(FName QualityLevelName);
	TArray<FName> GetMinQualityLevelLODOverrideNames() const;
	FReply ResetToDefault();

	void OnNoRefStreamingLODBiasChanged(int32 NewValue, FName QualityLevel);
	void OnNoRefStreamingLODBiasCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel);
	int32 GetNoRefStreamingLODBias(FName QualityLevel) const;
	TSharedRef<SWidget> GetNoRefStreamingLODBiasWidget(FName QualityLevelName) const;
	bool AddNoRefStreamingLODBiasOverride(FName QualityLevelName);
	bool RemoveNoRefStreamingLODBiasOverride(FName QualityLevelName);
	TArray<FName> GetNoRefStreamingLODBiasOverrideNames() const;

	void OnNumStreamedLODsChanged(int32 NewValue, FName Platform);
	void OnNumStreamedLODsCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName Platform);
	int32 GetNumStreamedLODs(FName Platform) const;
	TSharedRef<SWidget> GetNumStreamedLODsWidget(FName PlatformGroupName) const;
	bool AddNumStreamedLODsPlatformOverride(FName PlatformGroupName);
	bool RemoveNumStreamedLODsPlatformOverride(FName PlatformGroupName);
	TArray<FName> GetNumStreamedLODsPlatformOverrideNames() const;

	bool CanRemoveLOD(int32 LODIndex) const;
	FReply OnRemoveLOD(int32 LODIndex);

	float GetLODScreenSize(FName PlatformGroupName, int32 LODIndex)const;
	FText GetLODScreenSizeTitle(int32 LODIndex) const;
	bool CanChangeLODScreenSize() const;
	TSharedRef<SWidget> GetLODScreenSizeWidget(FName PlatformGroupName, int32 LODIndex) const;
	TArray<FName> GetLODScreenSizePlatformOverrideNames(int32 LODIndex) const;
	bool AddLODScreenSizePlatformOverride(FName PlatformGroupName, int32 LODIndex);
	bool RemoveLODScreenSizePlatformOverride(FName PlatformGroupName, int32 LODIndex);
	void OnLODScreenSizeChanged(float NewValue, FName PlatformGroupName, int32 LODIndex);
	void OnLODScreenSizeCommitted(float NewValue, ETextCommit::Type CommitType, FName PlatformGroupName, int32 LODIndex);
	float GetScreenSizeWidgetWidth(int32 LODIndex) const;

	FString GetSourceImportFilename(int32 LODIndex) const;
	void SetSourceImportFilename(const FString& SourceFileName, int32 LODIndex) const;

	void OnBuildSettingsExpanded(bool bIsExpanded, int32 LODIndex);
	void OnReductionSettingsExpanded(bool bIsExpanded, int32 LODIndex);
	void OnSectionSettingsExpanded(bool bIsExpanded, int32 LODIndex);
	void OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	bool IsAutoLODEnabled() const;
	ECheckBoxState IsAutoLODChecked() const;
	void OnAutoLODChanged(ECheckBoxState NewState);
	void OnImportLOD(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	void UpdateLODNames();
	FText GetLODCountTooltip() const;
	FText GetMinLODTooltip() const;
	FText GetNoRefStreamingLODBiasTooltip() const;
	FText GetNumStreamedLODsTooltip() const;

	FText GetLODCustomModeNameContent(int32 LODIndex) const;
	ECheckBoxState IsLODCustomModeCheck(int32 LODIndex) const;
	void SetLODCustomModeCheck(ECheckBoxState NewState, int32 LODIndex);
	bool IsLODCustomModeEnable(int32 LODIndex) const;

	TSharedRef<SWidget> OnGenerateLodComboBoxForLodPicker();
	EVisibility LodComboBoxVisibilityForLodPicker() const;
	bool IsLodComboBoxEnabledForLodPicker() const;
	TSharedRef<SWidget> OnGenerateLodMenuForLodPicker();
	FText GetCurrentLodName() const;
	FText GetCurrentLodTooltip() const;

private:

	/** The Static Mesh Editor this tool is associated with. */
	FStaticMeshEditor& StaticMeshEditor;

	/** LOD group options. */
	TArray<FName> LODGroupNames;
	TArray<TSharedPtr<FString> > LODGroupOptions;

	/** LOD import options */
	TArray<TSharedPtr<FString> > LODNames;

	/** Simplification options for each LOD level (in the LOD Chain tool). */
	TSharedPtr<FMeshReductionSettingsLayout> ReductionSettingsWidgets[MAX_STATIC_MESH_LODS];
	TSharedPtr<FMeshBuildSettingsLayout> BuildSettingsWidgets[MAX_STATIC_MESH_LODS];
	TSharedPtr<FMeshSectionSettingsLayout> SectionSettingsWidgets[MAX_STATIC_MESH_LODS];

	TSharedPtr<FMeshMaterialsLayout> MaterialsLayoutWidget;

	/** ComboBox widget for the LOD Group property */
	TSharedPtr<STextComboBox> LODGroupComboBox;

	/** The display factors at which LODs swap */
	FPerPlatformFloat LODScreenSizes[MAX_STATIC_MESH_LODS];

	/** Helper value that corresponds to the 'Number of LODs' spinbox.*/
	int32 LODCount;

	/** Whether certain parts of the UI are expanded so changes persist across
	    recreating the UI. */
	bool bBuildSettingsExpanded[MAX_STATIC_MESH_LODS];
	bool bReductionSettingsExpanded[MAX_STATIC_MESH_LODS];
	bool bSectionSettingsExpanded[MAX_STATIC_MESH_LODS];

	TArray<class IDetailCategoryBuilder*> LodCategories;
	IDetailCategoryBuilder* LodCustomCategory;

	bool DetailDisplayLODs[MAX_STATIC_MESH_LODS];

	FDelegateHandle OnAssetPostLODImportDelegateHandle;
};

/**
 * Window for Ray Tracing Proxy settings.
 */
class FRayTracingProxySettingsLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FRayTracingProxySettingsLayout>
{
public:
	FRayTracingProxySettingsLayout(FStaticMeshEditor& StaticMeshEditor);
	virtual ~FRayTracingProxySettingsLayout();

	const FMeshRayTracingProxySettings& GetSettings() const;
	void UpdateSettings(const FMeshRayTracingProxySettings& InSettings);

	/** Returns true if settings have been changed and an Apply is needed to update the asset. */
	bool IsApplyNeeded() const;

	/** Apply current Nanite settings to the mesh. */
	void ApplyChanges();

private:
	FReply OnApply();

	ECheckBoxState IsEnabledChecked() const;
	void OnEnabledChanged(ECheckBoxState NewState);

	float GetFallbackPercentTriangles() const;
	void OnFallbackPercentTrianglesChanged(float NewValue);
	void OnFallbackPercentTrianglesCommitted(float NewValue, ETextCommit::Type TextCommitType);

	float GetFallbackRelativeError() const;
	void OnFallbackRelativeErrorChanged(float NewValue);
	void OnFallbackRelativeErrorCommitted(float NewValue, ETextCommit::Type TextCommitType);

	float GetFoliageOverOcclusionBias() const;
	void OnFoliageOverOcclusionBiasChanged(float NewValue);
	void OnFoliageOverOcclusionBiasCommitted(float NewValue, ETextCommit::Type TextCommitType);

	bool ShouldEnable() const;
	bool CanToggleEnabled() const;

	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName RayTracingProxySettingsName("RayTracingProxySettings"); return RayTracingProxySettingsName; }
	virtual bool InitiallyCollapsed() const override { return true; }

private:
	/** The Static Mesh Editor this tool is associated with. */
	FStaticMeshEditor& StaticMeshEditor;

	FMeshRayTracingProxySettings RayTracingProxySettings;
};