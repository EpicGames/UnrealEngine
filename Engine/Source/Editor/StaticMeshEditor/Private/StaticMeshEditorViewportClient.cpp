// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorViewportClient.h"
#include "EditorModeManager.h"
#include "EngineGlobals.h"
#include "RawIndexBuffer.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/Canvas.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Engine/StaticMeshSocket.h"
#include "Utils.h"
#include "IStaticMeshEditor.h"
#include "UnrealEngine.h"

#include "StaticMeshResources.h"
#include "DistanceFieldAtlas.h"
#include "SEditorViewport.h"
#include "AdvancedPreviewScene.h"
#include "SStaticMeshEditorViewport.h"

#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "PhysicsEngine/BodySetup.h"

#include "Engine/AssetUserData.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "StaticMeshEditorTools.h"
#include "AssetViewerSettings.h"
#include "GizmoEdModeInterface.h"
#include "UnrealWidget.h"

#include "Rendering/NaniteResources.h"
#include "NaniteLayout.h"

#define LOCTEXT_NAMESPACE "FStaticMeshEditorViewportClient"

#define HITPROXY_SOCKET	1

namespace {
	static const float LightRotSpeed = 0.22f;
	static const float StaticMeshEditor_RotateSpeed = 0.01f;
	static const float	StaticMeshEditor_TranslateSpeed = 0.25f;
	static const float GridSize = 2048.0f;
	static const int32 CellSize = 16;
	static const float AutoViewportOrbitCameraTranslate = 256.0f;

	static float AmbientCubemapIntensity = 0.4f;
}

FStaticMeshEditorViewportClient::FStaticMeshEditorViewportClient(TWeakPtr<IStaticMeshEditor> InStaticMeshEditor, const TSharedRef<SStaticMeshEditorViewport>& InStaticMeshEditorViewport, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene, UStaticMesh* InPreviewStaticMesh, UStaticMeshComponent* InPreviewStaticMeshComponent)
	: FEditorViewportClient(nullptr, &InPreviewScene.Get(), StaticCastSharedRef<SEditorViewport>(InStaticMeshEditorViewport))
	, StaticMeshEditorPtr(InStaticMeshEditor)
	, StaticMeshEditorViewportPtr(InStaticMeshEditorViewport)
{
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(160,160,160);
	DrawHelper.GridColorMajor = FColor(144,144,144);
	DrawHelper.GridColorMinor = FColor(128,128,128);
	DrawHelper.PerspectiveGridSize = GridSize;
	DrawHelper.NumCells = FMath::FloorToInt32(DrawHelper.PerspectiveGridSize / (CellSize * 2));

	SetViewMode(VMI_Lit);

	WidgetMode = UE::Widget::WM_None;

	EngineShowFlags.SetSeparateTranslucency(true);
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);
	OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	bShowSimpleCollision = false;
	bShowComplexCollision = false;
	bShowSockets = true;
	bDrawUVs = false;
	bDrawNormals = false;
	bDrawTangents = false;
	bDrawBinormals = false;
	bShowPivot = false;
	bDrawAdditionalData = true;
	bDrawVertices = false;

	bManipulating = false;

	AdvancedPreviewScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);

	SetPreviewMesh(InPreviewStaticMesh, InPreviewStaticMeshComponent);
}

FStaticMeshEditorViewportClient::~FStaticMeshEditorViewportClient()
{
}

void FStaticMeshEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

/**
 * A hit proxy class for the wireframe collision geometry
 */
struct HSMECollisionProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	IStaticMeshEditor::FPrimData	PrimData;

	HSMECollisionProxy(const IStaticMeshEditor::FPrimData& InPrimData) :
		HHitProxy(HPP_UI),
		PrimData(InPrimData) {}

	HSMECollisionProxy(EAggCollisionShape::Type InPrimType, int32 InPrimIndex) :
		HHitProxy(HPP_UI),
		PrimData(InPrimType, InPrimIndex) {}
};
IMPLEMENT_HIT_PROXY(HSMECollisionProxy, HHitProxy);

/**
 * A hit proxy class for sockets.
 */
struct HSMESocketProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( );

	int32							SocketIndex;

	HSMESocketProxy(int32 InSocketIndex) :
		HHitProxy( HPP_UI ), 
		SocketIndex( InSocketIndex ) {}
};
IMPLEMENT_HIT_PROXY(HSMESocketProxy, HHitProxy);

/**
 * A hit proxy class for vertices.
 */
struct HSMEVertexProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	uint32		Index;

	HSMEVertexProxy(uint32 InIndex)
		: HHitProxy( HPP_UI )
		, Index( InIndex )
	{}
};
IMPLEMENT_HIT_PROXY(HSMEVertexProxy, HHitProxy);

bool FStaticMeshEditorViewportClient::InputWidgetDelta( FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale )
{
	bool bHandled = FEditorViewportClient::InputWidgetDelta(InViewport, CurrentAxis, Drag, Rot, Scale);

	if (!bHandled && bManipulating)
	{
		if (CurrentAxis != EAxisList::None)
		{
			const TArray<UStaticMeshSocket*> SelectedSockets = StaticMeshEditorPtr.Pin()->GetSelectedSockets();
			if (SelectedSockets.Num())
			{
				for (UStaticMeshSocket* SelectedSocket : SelectedSockets)
				{
					FProperty* ChangedProperty = NULL;
					const UE::Widget::EWidgetMode MoveMode = GetWidgetMode();
					if (MoveMode == UE::Widget::WM_Rotate)
					{
						ChangedProperty = FindFProperty<FProperty>(UStaticMeshSocket::StaticClass(), "RelativeRotation");
						SelectedSocket->PreEditChange(ChangedProperty);

						const FRotator CurrentRot = SelectedSocket->RelativeRotation;
						FRotator SocketWinding, SocketRotRemainder;
						CurrentRot.GetWindingAndRemainder(SocketWinding, SocketRotRemainder);

						const FQuat ActorQ = SocketRotRemainder.Quaternion();
						const FQuat DeltaQ = Rot.Quaternion();
						const FQuat ResultQ = DeltaQ * ActorQ;
						const FRotator NewSocketRotRem = FRotator(ResultQ);
						FRotator DeltaRot = NewSocketRotRem - SocketRotRemainder;
						DeltaRot.Normalize();

						SelectedSocket->RelativeRotation += DeltaRot;
						SelectedSocket->RelativeRotation = SelectedSocket->RelativeRotation.Clamp();
					}
					else if (MoveMode == UE::Widget::WM_Translate)
					{
						ChangedProperty = FindFProperty<FProperty>(UStaticMeshSocket::StaticClass(), "RelativeLocation");
						SelectedSocket->PreEditChange(ChangedProperty);

						//FRotationMatrix SocketRotTM( SelectedSocket->RelativeRotation );
						//FVector SocketMove = SocketRotTM.TransformVector( Drag );

						SelectedSocket->RelativeLocation += Drag;
					}
					if (ChangedProperty)
					{
						FPropertyChangedEvent PropertyChangedEvent(ChangedProperty);
						SelectedSocket->PostEditChangeProperty(PropertyChangedEvent);
					}

					StaticMeshEditorPtr.Pin()->GetStaticMesh()->MarkPackageDirty();
				}
			}
			else
			{
				const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->HasSelectedPrims();
				if (bSelectedPrim && CurrentAxis != EAxisList::None)
				{
					const UE::Widget::EWidgetMode MoveMode = GetWidgetMode();
					if (MoveMode == UE::Widget::WM_Rotate)
					{
						StaticMeshEditorPtr.Pin()->RotateSelectedPrims(Rot);
					}
					else if (MoveMode == UE::Widget::WM_Scale)
					{
						StaticMeshEditorPtr.Pin()->ScaleSelectedPrims(Scale);
					}
					else if (MoveMode == UE::Widget::WM_Translate)
					{
						StaticMeshEditorPtr.Pin()->TranslateSelectedPrims(Drag);
					}

					StaticMeshEditorPtr.Pin()->GetStaticMesh()->MarkPackageDirty();
				}
			}
		}

		Invalidate();		
		bHandled = true;
	}

	return bHandled;
}

void FStaticMeshEditorViewportClient::TrackingStarted( const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge )
{
	const bool bTrackingHandledExternally = ModeTools->StartTracking(this, Viewport);

	if( !bManipulating && bIsDraggingWidget && !bTrackingHandledExternally)
	{
		const bool bDuplicate = InInputState.IsLeftMouseButtonPressed()
			&& InInputState.IsAltButtonPressed()
			&& (Widget->GetCurrentAxis() & EAxisList::XYZ);
		
		BeginManipulation(bDuplicate);
	}

}

UE::Widget::EWidgetMode FStaticMeshEditorViewportClient::GetWidgetMode() const
{
	if (IsCustomModeUsingWidget())
	{
		return ModeTools->GetWidgetMode();
	}
	else if (StaticMeshEditorPtr.Pin()->HasSelectedSockets())
	{
		return WidgetMode;
	}
	else if (StaticMeshEditorPtr.Pin()->HasSelectedPrims())
	{
		return WidgetMode;
	}

	return UE::Widget::WM_Max;
}

void FStaticMeshEditorViewportClient::SetWidgetMode(UE::Widget::EWidgetMode NewMode)
{
	if (IsCustomModeUsingWidget())
	{
		ModeTools->SetWidgetMode(NewMode);
	}
	else
	{
		WidgetMode = NewMode;
	}

	Invalidate();
}

bool FStaticMeshEditorViewportClient::CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const
{
	if (!Widget->IsDragging())
	{
		if (IsCustomModeUsingWidget())
		{
			return ModeTools->UsesTransformWidget(NewMode);
		}
		else if (StaticMeshEditorPtr.Pin()->HasSelectedPrims())
		{
			return true;
		}
		else if (NewMode != UE::Widget::WM_Scale)	// Sockets don't support scaling
		{
			return StaticMeshEditorPtr.Pin()->HasSelectedSockets();
		}
	}
	return false;
}

bool FStaticMeshEditorViewportClient::CanCycleWidgetMode() const
{
	if (!Widget->IsDragging())
	{
		const bool bHasSelectedSockets = StaticMeshEditorPtr.Pin()->HasSelectedSockets();
		const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->HasSelectedPrims();
		if ((bHasSelectedSockets || bSelectedPrim || IsCustomModeUsingWidget()))
		{
			return true;
		}
	}
	return false;
}

void FStaticMeshEditorViewportClient::TrackingStopped()
{
	const bool bTrackingHandledExternally = ModeTools->EndTracking(this, Viewport);

	if( bManipulating && !bTrackingHandledExternally)
	{
		bManipulating = false;
		GEditor->EndTransaction();
	}
}

bool FStaticMeshEditorViewportClient::BeginTransform(const FGizmoState& InState)
{
	BeginManipulation(IsAltDown(Viewport) && GetWidgetMode() == UE::Widget::WM_Translate);
	return true;
}

void FStaticMeshEditorViewportClient::BeginManipulation(bool bDuplicate)
{
	Widget->SetSnapEnabled(true);
	const TArray<UStaticMeshSocket*> SelectedSockets = StaticMeshEditorPtr.Pin()->GetSelectedSockets();
	if (SelectedSockets.Num())
	{
		FText TransText;
		if( GetWidgetMode() == UE::Widget::WM_Rotate )
		{
			TransText = LOCTEXT("FStaticMeshEditorViewportClient_RotateSocket", "Rotate Socket");
		}
		else if (GetWidgetMode() == UE::Widget::WM_Translate)
		{
			if (bDuplicate)
			{
				// Rather than moving/rotating the selected socket, copy it and move the copy instead
				StaticMeshEditorPtr.Pin()->DuplicateSelectedSockets();
			}

			TransText = LOCTEXT("FStaticMeshEditorViewportClient_TranslateSocket", "Translate Socket");
		}

		if (!TransText.IsEmpty())
		{
			GEditor->BeginTransaction(TransText);
		}
	}
	
	const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->HasSelectedPrims();
	if (bSelectedPrim)
	{
		FText TransText;
		if (GetWidgetMode() == UE::Widget::WM_Rotate)
		{
			TransText = LOCTEXT("FStaticMeshEditorViewportClient_RotateCollision", "Rotate Collision");
		}
		else if (GetWidgetMode() == UE::Widget::WM_Scale)
		{
			TransText = LOCTEXT("FStaticMeshEditorViewportClient_ScaleCollision", "Scale Collision");
		}
		else if (GetWidgetMode() == UE::Widget::WM_Translate)
		{
			if (bDuplicate)
			{
				// Rather than moving/rotating the selected primitives, copy them and move the copies instead
				StaticMeshEditorPtr.Pin()->DuplicateSelectedPrims(NULL);
			}

			TransText = LOCTEXT("FStaticMeshEditorViewportClient_TranslateCollision", "Translate Collision");
		}
		if (!TransText.IsEmpty())
		{
			GEditor->BeginTransaction(TransText);
			if (StaticMesh->GetBodySetup())
			{
				StaticMesh->GetBodySetup()->Modify();
			}
		}
	}

	bManipulating = true;
}

bool FStaticMeshEditorViewportClient::EndTransform(const FGizmoState& InState)
{
	if (bManipulating)
	{
		bManipulating = false;
		GEditor->EndTransaction();
		return true;
	}
	return false;
}

FVector FStaticMeshEditorViewportClient::GetWidgetLocation() const
{
	if (IsCustomModeUsingWidget())
	{
		return ModeTools->GetWidgetLocation();
	}
	else if (StaticMeshEditorPtr.Pin()->HasSelectedSockets())
	{
		const UStaticMeshSocket* SelectedSocket = StaticMeshEditorPtr.Pin()->GetSelectedSockets()[0];
		FMatrix SocketTM;
		SelectedSocket->GetSocketMatrix(SocketTM, StaticMeshComponent);

		return SocketTM.GetOrigin();
	}

	FTransform PrimTransform = FTransform::Identity;
	const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->GetLastSelectedPrimTransform(PrimTransform);
	if (bSelectedPrim)
	{
		return PrimTransform.GetLocation();
	}
	else
	{
		StaticMeshEditorPtr.Pin()->ClearSelectedPrims();
		return FVector::ZeroVector;
	}
}

FMatrix FStaticMeshEditorViewportClient::GetWidgetCoordSystem() const 
{
	if (IsCustomModeUsingWidget())
	{
		return ModeTools->GetCustomInputCoordinateSystem();
	}

	if (StaticMeshEditorPtr.Pin()->HasSelectedSockets())
	{
		//FMatrix SocketTM;
		//SelectedSocket->GetSocketMatrix(SocketTM, StaticMeshComponent);
		const UStaticMeshSocket* SelectedSocket = StaticMeshEditorPtr.Pin()->GetSelectedSockets()[0];
		return FRotationMatrix(SelectedSocket->RelativeRotation);
	}

	FTransform PrimTransform = FTransform::Identity;
	const bool bSelectedPrim = StaticMeshEditorPtr.Pin()->GetLastSelectedPrimTransform(PrimTransform);
	if (bSelectedPrim)
	{
		return FRotationMatrix(PrimTransform.Rotator());
	}
	else
	{
		StaticMeshEditorPtr.Pin()->ClearSelectedPrims();
		return FMatrix::Identity;
	}
}

ECoordSystem FStaticMeshEditorViewportClient::GetWidgetCoordSystemSpace() const
{ 
	if (IsCustomModeUsingWidget())
	{
		return ModeTools->GetCoordSystem();
	}

	return COORD_Local; 
}

bool FStaticMeshEditorViewportClient::ShouldOrbitCamera() const
{
	if (GetDefault<ULevelEditorViewportSettings>()->bUseUE3OrbitControls)
	{
		// this editor orbits always if ue3 orbit controls are enabled
		return true;
	}

	return FEditorViewportClient::ShouldOrbitCamera();
}

void DrawCustomComplex(FPrimitiveDrawInterface* PDI, FTriMeshCollisionData Mesh, const FColor Color)
{
	for (int i = 0; i < Mesh.Indices.Num(); ++i)
	{
		PDI->DrawLine((FVector)Mesh.Vertices[Mesh.Indices[i].v0], (FVector)Mesh.Vertices[Mesh.Indices[i].v1], Color, SDPG_World);
		PDI->DrawLine((FVector)Mesh.Vertices[Mesh.Indices[i].v1], (FVector)Mesh.Vertices[Mesh.Indices[i].v2], Color, SDPG_World);
		PDI->DrawLine((FVector)Mesh.Vertices[Mesh.Indices[i].v2], (FVector)Mesh.Vertices[Mesh.Indices[i].v0], Color, SDPG_World);
	}
}

void FStaticMeshEditorViewportClient::Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

	TSharedPtr<IStaticMeshEditor> StaticMeshEditor = StaticMeshEditorPtr.Pin();

	if(!StaticMesh->GetRenderData() || !StaticMesh->GetRenderData()->LODResources.IsValidIndex(StaticMeshEditor->GetCurrentLODIndex()))
	{
		// Guard against corrupted meshes
		return;
	}

	UBodySetup* MeshBodySetup = StaticMesh->GetBodySetup();
	const bool bHasCookedPhysicsState = MeshBodySetup && !MeshBodySetup->bNeverNeedsCookedCollisionData;

	// Draw simple shapes if we are showing simple, or showing complex but using simple as complex
	if (StaticMesh->GetBodySetup() && bHasCookedPhysicsState && (bShowSimpleCollision || (bShowComplexCollision && StaticMesh->GetBodySetup()->CollisionTraceFlag == ECollisionTraceFlag::CTF_UseSimpleAsComplex)))
	{
		// Ensure physics mesh is created before we try and draw it
		StaticMesh->GetBodySetup()->CreatePhysicsMeshes();

		const FColor SelectedColor(20, 220, 20);
		const FColor UnselectedColor(0, 125, 0);

		const FVector VectorScaleOne(1.0f);

		// Draw bodies
		FKAggregateGeom* AggGeom = &StaticMesh->GetBodySetup()->AggGeom;

		for (int32 i = 0; i < AggGeom->SphereElems.Num(); ++i)
		{
			HSMECollisionProxy* HitProxy = new HSMECollisionProxy(EAggCollisionShape::Sphere, i);
			PDI->SetHitProxy(HitProxy);

			const FColor CollisionColor = StaticMeshEditor->IsSelectedPrim(HitProxy->PrimData) ? SelectedColor : UnselectedColor;
			const FKSphereElem& SphereElem = AggGeom->SphereElems[i];
			const FTransform ElemTM = SphereElem.GetTransform();
			SphereElem.DrawElemWire(PDI, ElemTM, VectorScaleOne, CollisionColor);

			PDI->SetHitProxy(NULL);
		}

		for (int32 i = 0; i < AggGeom->BoxElems.Num(); ++i)
		{
			HSMECollisionProxy* HitProxy = new HSMECollisionProxy(EAggCollisionShape::Box, i);
			PDI->SetHitProxy(HitProxy);

			const FColor CollisionColor = StaticMeshEditor->IsSelectedPrim(HitProxy->PrimData) ? SelectedColor : UnselectedColor;
			const FKBoxElem& BoxElem = AggGeom->BoxElems[i];
			const FTransform ElemTM = BoxElem.GetTransform();
			BoxElem.DrawElemWire(PDI, ElemTM, VectorScaleOne, CollisionColor);

			PDI->SetHitProxy(NULL);
		}

		for (int32 i = 0; i < AggGeom->SphylElems.Num(); ++i)
		{
			HSMECollisionProxy* HitProxy = new HSMECollisionProxy(EAggCollisionShape::Sphyl, i);
			PDI->SetHitProxy(HitProxy);

			const FColor CollisionColor = StaticMeshEditor->IsSelectedPrim(HitProxy->PrimData) ? SelectedColor : UnselectedColor;
			const FKSphylElem& SphylElem = AggGeom->SphylElems[i];
			const FTransform ElemTM = SphylElem.GetTransform();
			SphylElem.DrawElemWire(PDI, ElemTM, VectorScaleOne, CollisionColor);

			PDI->SetHitProxy(NULL);
		}

		for (int32 i = 0; i < AggGeom->ConvexElems.Num(); ++i)
		{
			HSMECollisionProxy* HitProxy = new HSMECollisionProxy(EAggCollisionShape::Convex, i);
			PDI->SetHitProxy(HitProxy);

			const FColor CollisionColor = StaticMeshEditor->IsSelectedPrim(HitProxy->PrimData) ? SelectedColor : UnselectedColor;
			const FKConvexElem& ConvexElem = AggGeom->ConvexElems[i];
			const FTransform ElemTM = ConvexElem.GetTransform();
			ConvexElem.DrawElemWire(PDI, ElemTM, 1.f, CollisionColor);

			PDI->SetHitProxy(NULL);
		}

		for (int32 i = 0; i < AggGeom->LevelSetElems.Num(); ++i)
		{
			HSMECollisionProxy* HitProxy = new HSMECollisionProxy(EAggCollisionShape::LevelSet, i);
			PDI->SetHitProxy(HitProxy);

			const FColor CollisionColor = StaticMeshEditor->IsSelectedPrim(HitProxy->PrimData) ? SelectedColor : UnselectedColor;
			const FKLevelSetElem& LevelSetElem = AggGeom->LevelSetElems[i];
			const FTransform ElemTM = LevelSetElem.GetTransform();
			LevelSetElem.DrawElemWire(PDI, ElemTM, 1.f, CollisionColor);

			PDI->SetHitProxy(NULL);
		}
	}

	if (bShowComplexCollision && bHasCookedPhysicsState && StaticMesh->ComplexCollisionMesh && StaticMesh->GetBodySetup()->CollisionTraceFlag != ECollisionTraceFlag::CTF_UseSimpleAsComplex)
	{
		const FColor UnselectedColor(0, 0, 125);

		// set the proxy to null to properly handle triangle meshes
		PDI->SetHitProxy(nullptr); 
		DrawCustomComplex(PDI, CollisionMeshData, UnselectedColor);
	}

	if( bShowSockets )
	{
		const FColor UnselectedSocketColor = FColor(255, 128, 128);
		const FColor SelectedSocketColor = FColor(255, 220, 220);

		for(int32 i=0; i < StaticMesh->Sockets.Num(); i++)
		{
			UStaticMeshSocket* Socket = StaticMesh->Sockets[i];
			if(Socket)
			{
				const bool bIsSocketSelected = StaticMeshEditor->GetSelectedSockets().Contains(Socket);

				FMatrix SocketTM;
				Socket->GetSocketMatrix(SocketTM, StaticMeshComponent);
				PDI->SetHitProxy( new HSMESocketProxy(i) );
				DrawWireDiamond(PDI, SocketTM, 5.f, bIsSocketSelected ? SelectedSocketColor : UnselectedSocketColor, SDPG_Foreground);
				PDI->SetHitProxy( NULL );
			}
		}
	}

	// Draw any edges that are currently selected by the user
	if( SelectedEdgeIndices.Num() > 0 )
	{
		for(int32 VertexIndex = 0; VertexIndex < SelectedEdgeVertices.Num(); VertexIndex += 2)
		{
			FVector EdgeVertices[ 2 ];
			EdgeVertices[ 0 ] = SelectedEdgeVertices[VertexIndex];
			EdgeVertices[ 1 ] = SelectedEdgeVertices[VertexIndex + 1];

			PDI->DrawLine(
				StaticMeshComponent->GetComponentTransform().TransformPosition( EdgeVertices[ 0 ] ),
				StaticMeshComponent->GetComponentTransform().TransformPosition( EdgeVertices[ 1 ] ),
				FColor( 255, 255, 0 ),
				SDPG_World );
		}
	}


	if( bDrawNormals || bDrawTangents || bDrawBinormals || bDrawVertices )
	{
		FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[StaticMeshEditor->GetCurrentLODIndex()];
		FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
		uint32 NumIndices = Indices.Num();

		FMatrix LocalToWorldInverseTranspose = StaticMeshComponent->GetComponentTransform().ToMatrixWithScale().InverseFast().GetTransposed();
		for (uint32 i = 0; i < NumIndices; i++)
		{
			const FVector3f& VertexPos = LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition( Indices[i] );

			const FVector WorldPos = StaticMeshComponent->GetComponentTransform().TransformPosition( (FVector)VertexPos );
			const FVector3f& Normal = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ( Indices[i] ); 
			const FVector3f& Binormal = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY( Indices[i] ); 
			const FVector3f& Tangent = LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX( Indices[i] ); 

			const float Len = 5.0f;
			const float BoxLen = 2.0f;
			const FVector Box(BoxLen);

			if( bDrawNormals )
			{
				PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( (FVector)Normal ).GetSafeNormal() * Len, FLinearColor( 0.0f, 1.0f, 0.0f), SDPG_World );
			}

			if( bDrawTangents )
			{
				PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( (FVector)Tangent ).GetSafeNormal() * Len, FLinearColor( 1.0f, 0.0f, 0.0f), SDPG_World );
			}

			if( bDrawBinormals )
			{
				PDI->DrawLine( WorldPos, WorldPos+LocalToWorldInverseTranspose.TransformVector( (FVector)Binormal ).GetSafeNormal() * Len, FLinearColor( 0.0f, 0.0f, 1.0f), SDPG_World );
			}

			if( bDrawVertices )
			{								
				PDI->SetHitProxy(new HSMEVertexProxy(i));
				DrawWireBox( PDI, FBox((FVector)VertexPos - Box, (FVector)VertexPos + Box), FLinearColor(0.0f, 1.0f, 0.0f), SDPG_World );
				PDI->SetHitProxy(NULL);								
			}
		}	
	}


	if( bShowPivot )
	{
		FUnrealEdUtils::DrawWidget(View, PDI, StaticMeshComponent->GetComponentTransform().ToMatrixWithScale(), 0, 0, EAxisList::All, EWidgetMovementMode::WMM_Translate, false);
	}

	if( bDrawAdditionalData )
	{
		const TArray<UAssetUserData*>* UserDataArray = StaticMesh->GetAssetUserDataArray();
		if (UserDataArray != NULL)
		{
			for (int32 AdditionalDataIndex = 0; AdditionalDataIndex < UserDataArray->Num(); ++AdditionalDataIndex)
			{
				if ((*UserDataArray)[AdditionalDataIndex] != NULL)
				{
					(*UserDataArray)[AdditionalDataIndex]->Draw(PDI, View);
				}
			}
		}

		if (StaticMesh->GetNavCollision()
			&& StaticMesh->bHasNavigationData)
		{
			// Draw the static mesh's body setup (simple collision)
			FTransform GeomTransform(StaticMeshComponent->GetComponentTransform());
			FColor NavCollisionColor = FColor(118, 84, 255, 255);
			StaticMesh->GetNavCollision()->DrawSimpleGeom(PDI, GeomTransform, FColorList::LimeGreen);
		}
	}
}

static void DrawAngles(FCanvas* Canvas, int32 XPos, int32 YPos, EAxisList::Type ManipAxis, UE::Widget::EWidgetMode MoveMode, const FRotator& Rotation, const FVector& Translation)
{
	FString OutputString(TEXT(""));
	if (MoveMode == UE::Widget::WM_Rotate && Rotation.IsZero() == false)
	{
		//Only one value moves at a time
		const FVector EulerAngles = Rotation.Euler();
		if (ManipAxis == EAxisList::X)
		{
			OutputString += FString::Printf(TEXT("Roll: %0.2f"), EulerAngles.X);
		}
		else if (ManipAxis == EAxisList::Y)
		{
			OutputString += FString::Printf(TEXT("Pitch: %0.2f"), EulerAngles.Y);
		}
		else if (ManipAxis == EAxisList::Z)
		{
			OutputString += FString::Printf(TEXT("Yaw: %0.2f"), EulerAngles.Z);
		}
	}
	else if (MoveMode == UE::Widget::WM_Translate && Translation.IsZero() == false)
	{
		//Only one value moves at a time
		if (ManipAxis == EAxisList::X)
		{
			OutputString += FString::Printf(TEXT(" %0.2f"), Translation.X);
		}
		else if (ManipAxis == EAxisList::Y)
		{
			OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Y);
		}
		else if (ManipAxis == EAxisList::Z)
		{
			OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Z);
		}
	}

	if (OutputString.Len() > 0)
	{
		FCanvasTextItem TextItem( FVector2D(XPos, YPos), FText::FromString( OutputString ), GEngine->GetSmallFont(), FLinearColor::White );
		Canvas->DrawItem( TextItem );
	}
}

void FStaticMeshEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	auto StaticMeshEditor = StaticMeshEditorPtr.Pin();
	auto StaticMeshEditorViewport = StaticMeshEditorViewportPtr.Pin();
	if (!StaticMeshEditor.IsValid() || !StaticMeshEditorViewport.IsValid() || !StaticMesh || StaticMesh->IsCompiling())
	{
		return;
	}

	const int32 HalfX = FMath::FloorToInt32(Viewport->GetSizeXY().X / 2 / GetDPIScale());
	const int32 HalfY = FMath::FloorToInt32(Viewport->GetSizeXY().Y / 2 / GetDPIScale());

	// Draw socket names if desired.
	if (bShowSockets)
	{
		for (int32 i = 0; i < StaticMesh->Sockets.Num(); i++)
		{
			UStaticMeshSocket* Socket = StaticMesh->Sockets[i];
			if (Socket != nullptr)
			{
				FMatrix SocketTM;
				Socket->GetSocketMatrix(SocketTM, StaticMeshComponent);
				const FVector SocketPos = SocketTM.GetOrigin();
				const FPlane proj = View.Project(SocketPos);
				if (proj.W > 0.f)
				{
					const int32 XPos = FMath::FloorToInt32( HalfX + (HalfX * proj.X) );
					const int32 YPos = FMath::FloorToInt32( HalfY + (HalfY * (proj.Y * -1)) );

					FCanvasTextItem TextItem(FVector2D(XPos, YPos), FText::FromString(Socket->SocketName.ToString()), GEngine->GetSmallFont(), FLinearColor(FColor(255, 196, 196)));
					Canvas.DrawItem(TextItem);

					if (bManipulating && StaticMeshEditor->GetSelectedSockets().Contains(Socket))
					{
						//Figure out the text height
						FTextSizingParameters Parameters(GEngine->GetSmallFont(), 1.0f, 1.0f);
						UCanvas::CanvasStringSize(Parameters, *Socket->SocketName.ToString());
						int32 YL = FMath::TruncToInt(Parameters.DrawYL);

						DrawAngles(&Canvas, XPos, YPos + YL,
							Widget->GetCurrentAxis(),
							GetWidgetMode(),
							Socket->RelativeRotation,
							Socket->RelativeLocation);
					}
				}
			}
		}
	}

	if( bDrawAdditionalData )
	{
		const TArray<UAssetUserData*>* UserDataArray = StaticMesh->GetAssetUserDataArray();
		if (UserDataArray != NULL)
		{
			for (int32 AdditionalDataIndex = 0; AdditionalDataIndex < UserDataArray->Num(); ++AdditionalDataIndex)
			{
				if ((*UserDataArray)[AdditionalDataIndex] != NULL)
				{
					(*UserDataArray)[AdditionalDataIndex]->DrawCanvas(Canvas, View);
				}
			}
		}
	}
	
	TArray<SStaticMeshEditorViewport::FOverlayTextItem, TInlineAllocator<10>> TextItems;

	const int32 CurrentLODLevel = [this, &StaticMeshEditor, &View]()
	{
		int32 LOD = StaticMeshEditor->GetCurrentLODLevel();
		return (LOD == 0) ?
			ComputeStaticMeshLOD(StaticMesh->GetRenderData(), StaticMeshComponent->Bounds.Origin, static_cast<float>(StaticMeshComponent->Bounds.SphereRadius), View, StaticMesh->GetDefaultMinLOD())
			:
			LOD - 1;
	}();

	if (StaticMesh->IsNaniteEnabled())
	{
		typedef Nanite::FSettingsLayout<UStaticMesh, true /* ForceEnable */, true /* HighRes */> TNaniteStaticMeshSettings;

		TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "NaniteEnabled", "<TextBlock.ShadowedText>Nanite Enabled</> <TextBlock.ShadowedTextWarning>{0}</>"), StaticMeshComponent->bDisplayNaniteFallbackMesh ? NSLOCTEXT("UnrealEd", "ShowingNaniteFallback", "(Showing Fallback)") : FText::GetEmpty()), false, true);

		if (StaticMesh->GetRenderData())
		{
			const Nanite::FResources& Resources = *StaticMesh->GetRenderData()->NaniteResourcesPtr.Get();
			if (Resources.RootData.Num() > 0)
			{
				const FString PositionStr = TNaniteStaticMeshSettings::PositionPrecisionValueToDisplayString(Resources.PositionPrecision);
				TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "NanitePositionPrecision", "Position Precision: {0}"), FText::FromString(PositionStr)));

				const FString NormalStr = TNaniteStaticMeshSettings::NormalPrecisionValueToDisplayString(Resources.NormalPrecision);
				TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "NaniteNormalPrecision", "Normal Precision: {0}"), FText::FromString(NormalStr)));

				const uint32 NumStreamingPages = Resources.PageStreamingStates.Num() - Resources.NumRootPages;
				const uint64 RootKB = uint64(Resources.NumRootPages) * NANITE_ROOT_PAGE_GPU_SIZE;
				const uint64 StreamingKB = uint64(NumStreamingPages) * NANITE_STREAMING_PAGE_GPU_SIZE;
				const uint64 TotalKB = RootKB + StreamingKB;

				FNumberFormattingOptions NumberOptions;
				NumberOptions.MinimumFractionalDigits = 2;
				NumberOptions.MaximumFractionalDigits = 2;

				TextItems.Emplace(FText::Format(
					NSLOCTEXT("UnrealEd", "NaniteResidency", "GPU Memory: Always allocated {0} MB. Streaming {1} MB. Total {2} MB."),
					FText::AsNumber(RootKB / 1048576.0f, &NumberOptions),
					FText::AsNumber(StreamingKB / 1048576.0f, &NumberOptions),
					FText::AsNumber(TotalKB / 1048576.0f, &NumberOptions)
				));
			}
		}
	}

	if (!StaticMesh->IsNaniteEnabled() || StaticMeshComponent->bDisplayNaniteFallbackMesh)
	{
		const int32 CurrentMinLODLevel = StaticMesh->GetMinLOD().GetValue();
		const bool bBelowMinLOD = CurrentLODLevel < CurrentMinLODLevel;

		TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "LOD_F", "LOD:  {0}"), FText::AsNumber(CurrentLODLevel)), bBelowMinLOD);

		if ( bBelowMinLOD )
		{
			TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "BelowMinLODWarning_F", "Selected LOD is below the minimum of {0}"),FText::AsNumber(CurrentMinLODLevel)), true);
		}
	}

	const float CurrentScreenSize = ComputeBoundsScreenSize(StaticMeshComponent->Bounds.Origin, static_cast<float>(StaticMeshComponent->Bounds.SphereRadius), View);
	FNumberFormattingOptions FormatOptions;
	FormatOptions.MinimumFractionalDigits = 3;
	FormatOptions.MaximumFractionalDigits = 6;
	FormatOptions.MaximumIntegralDigits = 6;
	StaticMeshEditorViewport->SetCurrentScreenPercentage(CurrentScreenSize);

	TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "ScreenSize_F", "Current Screen Size:  {0}"), FText::AsNumber(CurrentScreenSize, &FormatOptions)));

	const FText StaticMeshTriangleCount = FText::AsNumber(StaticMeshEditorPtr.Pin()->GetNumTriangles(CurrentLODLevel));
	const FText StaticMeshVertexCount = FText::AsNumber(StaticMeshEditorPtr.Pin()->GetNumVertices(CurrentLODLevel));

	if (StaticMesh->IsNaniteEnabled() && StaticMesh->GetRenderData())
	{
		const Nanite::FResources& Resources = *StaticMesh->GetRenderData()->NaniteResourcesPtr.Get();
		if (Resources.RootData.Num() > 0)
		{
			// Nanite Mesh Information
			const FText NaniteTriangleCount = FText::AsNumber(Resources.NumInputTriangles);
			const FText NaniteVertexCount = FText::AsNumber(Resources.NumInputVertices);

			TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Nanite_Triangles_F", "Nanite Triangles:  {0}"), NaniteTriangleCount));
			TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Nanite_Vertices_F", "Nanite Vertices:  {0}"), NaniteVertexCount));

			// Fallback Mesh Information
			TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Fallback_Triangles_F", "Fallback Triangles:  {0}"), StaticMeshTriangleCount));
			TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Fallback_Vertices_F", "Fallback Vertices:  {0}"), StaticMeshVertexCount));

			// Ray Tracing Fallback Mesh Information
			const FStaticMeshRayTracingProxy* RayTracingProxy = StaticMesh->GetRenderData()->RayTracingProxy;
			if (RayTracingProxy && !RayTracingProxy->bUsingRenderingLODs && RayTracingProxy->LODs.Num() > 0)
			{
				const int32 PreviewLODIndex = FMath::Clamp(RayTracingProxy->PreviewLODLevel, 0, RayTracingProxy->LODs.Num() - 1);
				const FStaticMeshRayTracingProxyLOD& PreviewLOD = RayTracingProxy->LODs[PreviewLODIndex];
				const FStaticMeshVertexBuffers* VertexBuffers = RayTracingProxy->LODs[PreviewLODIndex].VertexBuffers;
				const FRawStaticIndexBuffer* IndexBuffer = RayTracingProxy->LODs[PreviewLODIndex].IndexBuffer;

				uint32 RayTracingFallbackTriangleCount = 0;
				uint32 RayTracingFallbackVertexCount = 0;

				if (VertexBuffers && IndexBuffer)
				{
					RayTracingFallbackTriangleCount = IndexBuffer->GetNumIndices() / 3;
					RayTracingFallbackVertexCount = VertexBuffers->PositionVertexBuffer.GetNumVertices();
				}

				if (RayTracingFallbackTriangleCount > 0 && RayTracingFallbackVertexCount > 0)
				{
					TextItems.Emplace(FText::Format(
						NSLOCTEXT("UnrealEd", "RayTracingFallback_Triangles_F", "Ray Tracing Fallback Triangles:  {0}"),
						FText::AsNumber(RayTracingFallbackTriangleCount)));
					TextItems.Emplace(FText::Format(
						NSLOCTEXT("UnrealEd", "RayTracingFallback_Vertices_F", "Ray Tracing Fallback Vertices:  {0}"),
						FText::AsNumber(RayTracingFallbackVertexCount)));
				}
			}
		}
	}
	else
	{
		TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Triangles_F", "Triangles:  {0}"), StaticMeshTriangleCount));
		TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "Vertices_F", "Vertices:  {0}"), StaticMeshVertexCount));
	}

	TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "UVChannels_F", "UV Channels:  {0}"), FText::AsNumber(StaticMeshEditorPtr.Pin()->GetNumUVChannels(CurrentLODLevel))));

	if (StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->LODResources.Num() > 0)
	{
		if (StaticMesh->GetRenderData()->LODResources[0].DistanceFieldData != nullptr )
		{
			const FDistanceFieldVolumeData& VolumeData = *(StaticMesh->GetRenderData()->LODResources[0].DistanceFieldData);
			const FIntVector VolumeSize = VolumeData.Mips[0].IndirectionDimensions * DistanceField::UniqueDataBrickSize;
			{
				float AlwaysLoadedMemoryMb = VolumeData.GetResourceSizeBytes() / (1024.0f * 1024.0f);
				float HighestResMipMemoryMb = VolumeData.Mips[0].BulkSize / (1024.0f * 1024.0f);

				FNumberFormattingOptions NumberOptions;
				NumberOptions.MinimumFractionalDigits = 2;
				NumberOptions.MaximumFractionalDigits = 2;

				TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "DistanceFieldRes_F", "Distance Field:  {0}x{1}x{2} = {3}Mb always loaded, {4}Mb streamed"), FText::AsNumber(VolumeSize.X), FText::AsNumber(VolumeSize.Y), FText::AsNumber(VolumeSize.Z), FText::AsNumber(AlwaysLoadedMemoryMb, &NumberOptions), FText::AsNumber(HighestResMipMemoryMb, &NumberOptions)));
			}
		}
	}

	TextItems.Emplace(
		FText::Format(NSLOCTEXT("UnrealEd", "ApproxSize_F", "Approx Size: {0}x{1}x{2}"),
		FText::AsNumber(int32(StaticMesh->GetBounds().BoxExtent.X * 2.0f)), // x2 as artists wanted length not radius
		FText::AsNumber(int32(StaticMesh->GetBounds().BoxExtent.Y * 2.0f)),
		FText::AsNumber(int32(StaticMesh->GetBounds().BoxExtent.Z * 2.0f))));

	// Show the number of collision primitives
	if (StaticMesh->GetBodySetup())
	{
		UBodySetup* MeshBodySetup = StaticMesh->GetBodySetup();
		TextItems.Emplace(FText::Format(NSLOCTEXT("UnrealEd", "NumPrimitives_F", "Num Collision Primitives:  {0}"), FText::AsNumber(MeshBodySetup->AggGeom.GetElementCount())));

		if(MeshBodySetup->bNeverNeedsCookedCollisionData)
		{
			TextItems.Emplace(NSLOCTEXT("UnrealEd", "NumPrimitivesCookWarning", "\tbNeverNeedsCookedCollisionData Enabled, no collision data will be used."), true);
		}
	}

	// Estimated compressed size
	if (StaticMesh->GetRenderData())
	{
		FNumberFormattingOptions NumberOptions;
		NumberOptions.MinimumFractionalDigits = 2;
		NumberOptions.MaximumFractionalDigits = 2;

		TextItems.Emplace(
			FText::Format(NSLOCTEXT("UnrealEd", "EstimatedCompressedSize", "Estimated Compressed Disk Size: {0} MB ({1} MB Nanite)"),
			FText::AsNumber(StaticMesh->GetRenderData()->EstimatedCompressedSize / 1048576.0f, &NumberOptions),
			FText::AsNumber(StaticMesh->GetRenderData()->EstimatedNaniteTotalCompressedSize / 1048576.0f, &NumberOptions)));
	}

	if (StaticMeshComponent && StaticMeshComponent->SectionIndexPreview != INDEX_NONE)
	{
		TextItems.Emplace(NSLOCTEXT("UnrealEd", "MeshSectionsHiddenWarning",  "Mesh Sections Hidden"));
	}

	StaticMeshEditorViewport->PopulateOverlayText(MakeArrayView(TextItems));

 	int32 X = Canvas.GetRenderTarget()->GetSizeXY().X - 300;
 	int32 Y = 30;

	if (StaticMesh->GetBodySetup() && (!(StaticMesh->GetBodySetup()->bHasCookedCollisionData || StaticMesh->GetBodySetup()->bNeverNeedsCookedCollisionData) || StaticMesh->GetBodySetup()->bFailedToCreatePhysicsMeshes))
	{
		static const FText Message = NSLOCTEXT("Renderer", "NoCookedCollisionObject", "NO COOKED COLLISION OBJECT: TOO SMALL?");
		Canvas.DrawShadowedText(static_cast<float>(X), static_cast<float>(Y), Message, GetStatsFont(), FLinearColor(1.0f, 0.05f, 0.05f, 1.0f));
	}

	if (bDrawUVs && StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->LODResources.Num() > 0)
	{
		const int32 YPos = 160;
		DrawUVsForMesh(Viewport, &Canvas, YPos);
	}

	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
}

void FStaticMeshEditorViewportClient::DrawUVsForMesh(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos )
{
	//use the overridden LOD level
	const uint32 LODLevel = FMath::Clamp(StaticMeshComponent->ForcedLodModel - 1, 0, StaticMesh->GetRenderData()->LODResources.Num() - 1);

	int32 UVChannel = StaticMeshEditorPtr.Pin()->GetCurrentUVChannel();

	DrawUVs(InViewport, InCanvas, InTextYPos, LODLevel, UVChannel, SelectedEdgeTexCoords[UVChannel], StaticMeshComponent->GetStaticMesh()->GetRenderData(), NULL);
}

void FStaticMeshEditorViewportClient::MouseMove(FViewport* InViewport,int32 x, int32 y)
{
	FEditorViewportClient::MouseMove(InViewport,x,y);
}

bool FStaticMeshEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = FEditorViewportClient::InputKey(EventArgs);

	// Handle viewport screenshot.
	bHandled |= InputTakeScreenshot( EventArgs.Viewport, EventArgs.Key, EventArgs.Event );

	bHandled |= AdvancedPreviewScene->HandleInputKey(EventArgs);

	return bHandled;
}

bool FStaticMeshEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	bool bResult = true;
	
	if (!bDisableInput)
	{
		bResult = AdvancedPreviewScene->HandleViewportInput(Args.Viewport, Args.InputDevice, Args.Key, Args.AmountDepressed, Args.DeltaTime, Args.NumSamples, Args.IsGamepad());
		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(Args);
		}
	}

	return bResult;
}

void FStaticMeshEditorViewportClient::ProcessClick(class FSceneView& InView, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	const bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);

	bool ClearSelectedSockets = true;
	bool ClearSelectedPrims = true;
	bool ClearSelectedEdges = true;

	if( HitProxy )
	{
		if(HitProxy->IsA( HSMESocketProxy::StaticGetType() ) )
		{
			HSMESocketProxy* SocketProxy = (HSMESocketProxy*)HitProxy;

			UStaticMeshSocket* Socket = NULL;

			if(SocketProxy->SocketIndex < StaticMesh->Sockets.Num())
			{
				Socket = StaticMesh->Sockets[SocketProxy->SocketIndex];
			}

			if(Socket)
			{
				if (!bCtrlDown)
				{
					StaticMeshEditorPtr.Pin()->SetSelectedSocket(Socket);
				}
				else
				{
					//Select the socket if it is unselected and vice versa
					if (StaticMeshEditorPtr.Pin()->GetSelectedSockets().Contains(Socket))
					{
						StaticMeshEditorPtr.Pin()->RemoveSelectedSocket(Socket);
					}
					else
					{
						StaticMeshEditorPtr.Pin()->AddSelectedSocket(Socket);
					}
				}
			}

			ClearSelectedSockets = false;
		}
		else if (HitProxy->IsA(HSMECollisionProxy::StaticGetType()) && StaticMesh->GetBodySetup())
		{
			HSMECollisionProxy* CollisionProxy = (HSMECollisionProxy*)HitProxy;			

			if (StaticMeshEditorPtr.Pin()->IsSelectedPrim(CollisionProxy->PrimData))
			{
				if (!bCtrlDown)
				{
					StaticMeshEditorPtr.Pin()->AddSelectedPrim(CollisionProxy->PrimData, true);
				}
				else
				{
					StaticMeshEditorPtr.Pin()->RemoveSelectedPrim(CollisionProxy->PrimData);
				}
			}
			else
			{
				StaticMeshEditorPtr.Pin()->AddSelectedPrim(CollisionProxy->PrimData, !bCtrlDown);
			}

			// Force the widget to translate, if not already set
			if (WidgetMode == UE::Widget::WM_None)
			{
				WidgetMode = UE::Widget::WM_Translate;
			}

			ClearSelectedPrims = false;
		}
		else if (IsShowSocketsChecked() && HitProxy->IsA(HSMEVertexProxy::StaticGetType()))
		{
			TArray<UStaticMeshSocket*> Sockets = StaticMeshEditorPtr.Pin()->GetSelectedSockets();

			for (UStaticMeshSocket* Socket : Sockets)
			{
				HSMEVertexProxy* VertexProxy = (HSMEVertexProxy*)HitProxy;
				TSharedPtr<IStaticMeshEditor> StaticMeshEditor = StaticMeshEditorPtr.Pin();
				if (StaticMeshEditor.IsValid())
				{
					FStaticMeshLODResources& LODModel = StaticMesh->GetRenderData()->LODResources[StaticMeshEditor->GetCurrentLODIndex()];
					FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
					const uint32 Index = Indices[VertexProxy->Index];

					Socket->RelativeLocation = (FVector)LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(Index);
					Socket->RelativeRotation = FRotator(FRotationMatrix44f::MakeFromYZ(LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index), LODModel.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(Index)).Rotator());

					ClearSelectedSockets = false;
				}
			}
		}
	}
	else
	{
		const bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

		if(!bCtrlDown && !bShiftDown)
		{
			SelectedEdgeIndices.Empty();
		}

		// Check to see if we clicked on a mesh edge
		if( StaticMeshComponent != NULL && Viewport->GetSizeXY().X > 0 && Viewport->GetSizeXY().Y > 0 )
		{
			FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( Viewport, GetScene(), EngineShowFlags ));
			FSceneView* View = CalcSceneView(&ViewFamily);
			FViewportClick ViewportClick(View, this, Key, Event, HitX, HitY);

			const FVector ClickLineStart( ViewportClick.GetOrigin() );
			const FVector ClickLineEnd( ViewportClick.GetOrigin() + ViewportClick.GetDirection() * HALF_WORLD_MAX );

			// Don't bother doing a line check as there is only one mesh in the SME and it makes fuzzy selection difficult
			// 	FHitResult CheckResult( 1.0f );
			// 	if( StaticMeshComponent->LineCheck(
			// 			CheckResult,	// In/Out: Result
			// 			ClickLineEnd,	// Target
			// 			ClickLineStart,	// Source
			// 			FVector::ZeroVector,	// Extend
			// 			TRACE_ComplexCollision ) )	// Trace flags
			{
				// @todo: Should be in screen space ideally
				const float WorldSpaceMinClickDistance = 100.0f;

				float ClosestEdgeDistance = FLT_MAX;
				TArray< int32 > ClosestEdgeIndices;
				FVector ClosestEdgeVertices[ 2 ];

				const uint32 LODLevel = FMath::Clamp( StaticMeshComponent->ForcedLodModel - 1, 0, StaticMeshComponent->GetStaticMesh()->GetNumLODs() - 1 );
				if (StaticMeshComponent->GetStaticMesh()->HasValidRenderData(true, LODLevel))
				{
					FStaticMeshLODResources& RenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData()->LODResources[LODLevel];

					int32 NumBackFacingTriangles = 0;
					for (int32 SectionIndex = 0; SectionIndex < RenderData.Sections.Num(); ++SectionIndex)
					{
						const FStaticMeshSection& Section = RenderData.Sections[SectionIndex];
						const int32 FaceMaterialIndex = Section.MaterialIndex;
						const int32 NumFaces = Section.NumTriangles;
						uint32 IndexBufferIndex = Section.FirstIndex;
						for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
						{
							FVector VertexPosition[3];
							uint32 VertexIndex[3];
							uint32 WedgeIndex[3];
							for (int32 Corner = 0; Corner < 3; ++Corner)
							{
								WedgeIndex[Corner] = IndexBufferIndex;
								VertexIndex[Corner] = RenderData.IndexBuffer.GetIndex(IndexBufferIndex);
								VertexPosition[Corner] = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex[Corner]);
								IndexBufferIndex++;
							}
							// We disable edge selection where all adjoining triangles are back face culled and the 
							// material is not two-sided. This prevents edges that are back-face culled from being selected.
							bool bIsBackFacing = false;
							bool bIsTwoSided = false;
							UMaterialInterface* Material = StaticMeshComponent->GetMaterial(FaceMaterialIndex);
							if (Material && Material->GetMaterial())
							{
								bIsTwoSided = Material->IsTwoSided();
							}
							if (!bIsTwoSided)
							{
								// Check whether triangle if back facing 
								const FVector A = VertexPosition[0];
								const FVector B = VertexPosition[1];
								const FVector C = VertexPosition[2];

								// Compute the per-triangle normal
								const FVector BA = A - B;
								const FVector CA = A - C;
								const FVector TriangleNormal = (CA ^ BA).GetSafeNormal();

								// Transform the view position from world to component space
								const FVector ComponentSpaceViewOrigin = StaticMeshComponent->GetComponentTransform().InverseTransformPosition(View->ViewMatrices.GetViewOrigin());

								// Determine which side of the triangle's plane that the view position lies on.
								bIsBackFacing = (FVector::PointPlaneDist(ComponentSpaceViewOrigin, A, TriangleNormal) < 0.0f);
							}

							for (int32 Corner = 0; Corner < 3; ++Corner)
							{
								const int32 Corner2 = (Corner + 1) % 3;

								FVector EdgeVertices[2];
								EdgeVertices[0] = VertexPosition[Corner];
								EdgeVertices[1] = VertexPosition[Corner2];

								// First check to see if this edge is already in our "closest to click" list.
								// Most edges are shared by two faces in our raw triangle data set, so we want
								// to select (or deselect) both of these edges that the user clicks on (what
								// appears to be) a single edge
								if (ClosestEdgeIndices.Num() > 0 &&
									((EdgeVertices[0].Equals(ClosestEdgeVertices[0]) && EdgeVertices[1].Equals(ClosestEdgeVertices[1])) ||
									(EdgeVertices[0].Equals(ClosestEdgeVertices[1]) && EdgeVertices[1].Equals(ClosestEdgeVertices[0]))))
								{
									// Edge overlaps the closest edge we have so far, so just add it to the list
									ClosestEdgeIndices.Add(WedgeIndex[Corner]);
									// Increment the number of back facing triangles if the adjoining triangle 
									// is back facing and isn't two-sided
									if (bIsBackFacing && !bIsTwoSided)
									{
										++NumBackFacingTriangles;
									}
								}
								else
								{
									FVector WorldSpaceEdgeStart(StaticMeshComponent->GetComponentTransform().TransformPosition(EdgeVertices[0]));
									FVector WorldSpaceEdgeEnd(StaticMeshComponent->GetComponentTransform().TransformPosition(EdgeVertices[1]));

									// Determine the mesh edge that's closest to the ray cast through the eye towards the click location
									FVector ClosestPointToEdgeOnClickLine;
									FVector ClosestPointToClickLineOnEdge;
									FMath::SegmentDistToSegment(
										ClickLineStart,
										ClickLineEnd,
										WorldSpaceEdgeStart,
										WorldSpaceEdgeEnd,
										ClosestPointToEdgeOnClickLine,
										ClosestPointToClickLineOnEdge);

									// Compute the minimum distance (squared)
									const float MinDistanceToEdgeSquared = static_cast<float>( (ClosestPointToClickLineOnEdge - ClosestPointToEdgeOnClickLine).SizeSquared() );

									if (MinDistanceToEdgeSquared <= WorldSpaceMinClickDistance)
									{
										if (MinDistanceToEdgeSquared <= ClosestEdgeDistance)
										{
											// This is the closest edge to the click line that we've found so far!
											ClosestEdgeDistance = MinDistanceToEdgeSquared;
											ClosestEdgeVertices[0] = EdgeVertices[0];
											ClosestEdgeVertices[1] = EdgeVertices[1];

											ClosestEdgeIndices.Reset();
											ClosestEdgeIndices.Add(WedgeIndex[Corner]);

											// Reset the number of back facing triangles.
											NumBackFacingTriangles = (bIsBackFacing && !bIsTwoSided) ? 1 : 0;
										}
									}
								}
							}
						}
					}


					// Did the user click on an edge? Edges must also have at least one adjoining triangle 
					// which isn't back face culled (for one-sided materials)
					if (ClosestEdgeIndices.Num() > 0 && ClosestEdgeIndices.Num() > NumBackFacingTriangles)
					{
						for (int32 CurIndex = 0; CurIndex < ClosestEdgeIndices.Num(); ++CurIndex)
						{
							const int32 CurEdgeIndex = ClosestEdgeIndices[CurIndex];

							if (bCtrlDown)
							{
								// Toggle selection
								if (SelectedEdgeIndices.Contains(CurEdgeIndex))
								{
									SelectedEdgeIndices.Remove(CurEdgeIndex);
								}
								else
								{
									SelectedEdgeIndices.Add(CurEdgeIndex);
								}
							}
							else
							{
								// Append to selection
								SelectedEdgeIndices.Add(CurEdgeIndex);
							}
						}

						// Reset cached vertices and uv coordinates.
						SelectedEdgeVertices.Reset();
						for (int32 TexCoordIndex = 0; TexCoordIndex < MAX_STATIC_TEXCOORDS; ++TexCoordIndex)
						{
							SelectedEdgeTexCoords[TexCoordIndex].Reset();
						}

						for (FSelectedEdgeSet::TIterator SelectionIt(SelectedEdgeIndices); SelectionIt; ++SelectionIt)
						{
							const uint32 EdgeIndex = *SelectionIt;
							const uint32 FaceIndex = EdgeIndex / 3;

							const uint32 WedgeIndex = FaceIndex * 3 + (EdgeIndex % 3);
							const uint32 WedgeIndex2 = FaceIndex * 3 + ((EdgeIndex + 1) % 3);

							const uint32 VertexIndex = RenderData.IndexBuffer.GetIndex(WedgeIndex);
							const uint32 VertexIndex2 = RenderData.IndexBuffer.GetIndex(WedgeIndex2);
							// Cache edge vertices in local space.
							FVector EdgeVertices[2];
							EdgeVertices[0] = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
							EdgeVertices[1] = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex2);

							SelectedEdgeVertices.Add(EdgeVertices[0]);
							SelectedEdgeVertices.Add(EdgeVertices[1]);

							// Cache UV
							for (uint32 TexCoordIndex = 0; TexCoordIndex < MAX_STATIC_TEXCOORDS; ++TexCoordIndex)
							{
								if (RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() > TexCoordIndex)
								{
									FVector2D UVIndex1, UVIndex2;
									UVIndex1 = FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, TexCoordIndex));
									UVIndex2 = FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex2, TexCoordIndex));
									SelectedEdgeTexCoords[TexCoordIndex].Add(UVIndex1);
									SelectedEdgeTexCoords[TexCoordIndex].Add(UVIndex2);
								}
							}
						}

						ClearSelectedEdges = false;
					}
				}
			}
		}
	}

	if (ClearSelectedSockets && StaticMeshEditorPtr.Pin()->HasSelectedSockets())
	{
		StaticMeshEditorPtr.Pin()->SetSelectedSocket(NULL);
	}
	if (ClearSelectedPrims)
	{
		StaticMeshEditorPtr.Pin()->ClearSelectedPrims();
	}
	if (ClearSelectedEdges)
	{
		SelectedEdgeIndices.Empty();
		SelectedEdgeVertices.Empty();
		for (int32 TexCoordIndex = 0; TexCoordIndex < MAX_STATIC_TEXCOORDS; ++TexCoordIndex)
		{
			SelectedEdgeTexCoords[TexCoordIndex].Empty();
		}
	}

	Invalidate();
}

void FStaticMeshEditorViewportClient::PerspectiveCameraMoved()
{
	FEditorViewportClient::PerspectiveCameraMoved();

	// If in the process of transitioning to a new location, don't update the orbit camera position.
	// On the final update of the transition, we will get here with IsPlaying()==false, and the editor camera position will
	// be correctly updated.
	if (GetViewTransform().IsPlaying())
	{
		return;
	}

	// The static mesh editor saves the camera position in terms of an orbit camera, so ensure 
	// that orbit mode is enabled before we store the current transform information.
	const bool bWasOrbit = bUsingOrbitCamera;
	const FVector OldCameraLocation = GetViewLocation();
	const FRotator OldCameraRotation = GetViewRotation();
	ToggleOrbitCamera(true);

	const FVector OrbitPoint = GetLookAtLocation();
	const FVector OrbitZoom = GetViewLocation() - OrbitPoint;
	StaticMesh->SetEditorCameraPosition(FAssetEditorOrbitCameraPosition(
		OrbitPoint,
		OrbitZoom,
		GetViewRotation()
		));

	ToggleOrbitCamera(bWasOrbit);
}

void FStaticMeshEditorViewportClient::UpdateSimpleCollisionDisplay()
{
	if (StaticMeshComponent != nullptr)
	{
		// Have to set this flag in case we are using 'use complex as simple'
		StaticMeshComponent->bDrawMeshCollisionIfSimple = bShowSimpleCollision;
		StaticMeshComponent->MarkRenderStateDirty();
	}

	Invalidate();
}

void FStaticMeshEditorViewportClient::UpdateComplexCollisionDisplay()
{
	if (StaticMesh)
	{
		if (UObject* CDPObj = StaticMesh->ComplexCollisionMesh)
		{
			if (IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(CDPObj))
			{
				CollisionMeshData = FTriMeshCollisionData();
				CDP->GetPhysicsTriMeshData(&CollisionMeshData, true);
			}
			if (StaticMeshComponent != nullptr)
			{
				StaticMeshComponent->bDrawMeshCollisionIfComplex = false;
				StaticMeshComponent->MarkRenderStateDirty();
			}
		}
		else if (StaticMeshComponent != nullptr)
		{
			StaticMeshComponent->bDrawMeshCollisionIfComplex = bShowComplexCollision;
			StaticMeshComponent->MarkRenderStateDirty();
		}
	}

	Invalidate();
}

void FStaticMeshEditorViewportClient::SetFloorAndEnvironmentVisibility(const bool bVisible)
{
	AdvancedPreviewScene->SetFloorVisibility(bVisible, true);
	AdvancedPreviewScene->SetEnvironmentVisibility(bVisible, true);
}

void FStaticMeshEditorViewportClient::SetPreviewMesh(UStaticMesh* InStaticMesh, UStaticMeshComponent* InStaticMeshComponent, bool bResetCamera)
{
	StaticMesh = InStaticMesh;
	StaticMeshComponent = InStaticMeshComponent;

	UpdateSimpleCollisionDisplay();
	UpdateComplexCollisionDisplay();
	
	if (bResetCamera)
	{
		// If we have a thumbnail transform, we will favor that over the camera position as the user may have customized this for a nice view
		// If we have neither a custom thumbnail nor a valid camera position, then we'll just use the default thumbnail transform 
		const USceneThumbnailInfo* const AssetThumbnailInfo = Cast<USceneThumbnailInfo>(StaticMesh->GetThumbnailInfo());
		const USceneThumbnailInfo* const DefaultThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();

		// Prefer the asset thumbnail if available
		const USceneThumbnailInfo* const ThumbnailInfo = (AssetThumbnailInfo) ? AssetThumbnailInfo : DefaultThumbnailInfo;
		check(ThumbnailInfo);

		FRotator ThumbnailAngle;
		ThumbnailAngle.Pitch = ThumbnailInfo->OrbitPitch;
		ThumbnailAngle.Yaw = ThumbnailInfo->OrbitYaw;
		ThumbnailAngle.Roll = 0;
		const float ThumbnailDistance = ThumbnailInfo->OrbitZoom;

		const float CameraY = static_cast<float>( StaticMesh->GetBounds().SphereRadius / (75.0f * PI / 360.0f) );
		SetCameraSetup(
			FVector::ZeroVector,
			ThumbnailAngle,
			FVector(0.0f, CameraY + ThumbnailDistance - AutoViewportOrbitCameraTranslate, 0.0f),
			StaticMesh->GetBounds().Origin,
			-FVector(0, CameraY, 0),
			FRotator(0, 90.f, 0)
			);

		if (!AssetThumbnailInfo && StaticMesh->GetEditorCameraPosition().bIsSet)
		{
			// The static mesh editor saves the camera position in terms of an orbit camera, so ensure 
			// that orbit mode is enabled before we set the new transform information
			const bool bWasOrbit = bUsingOrbitCamera;
			ToggleOrbitCamera(true);

			SetViewRotation(StaticMesh->GetEditorCameraPosition().CamOrbitRotation);
			SetViewLocation(StaticMesh->GetEditorCameraPosition().CamOrbitPoint + StaticMesh->GetEditorCameraPosition().CamOrbitZoom);
			SetLookAtLocation(StaticMesh->GetEditorCameraPosition().CamOrbitPoint);

			ToggleOrbitCamera(bWasOrbit);
		}
	}
}

void FStaticMeshEditorViewportClient::ToggleShowNormals()
{
	bDrawNormals = !bDrawNormals;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawNormals"), bDrawNormals ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}


void FStaticMeshEditorViewportClient::SetShowNormals(bool bShowOn)
{
	bDrawNormals = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawNormals"), bDrawNormals ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowTangents(bool bShowOn)
{
	bDrawTangents = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawTangents"), bDrawTangents ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowBinormals(bool bShowOn)
{
	bDrawBinormals = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawBinormals"), bDrawBinormals ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowSimpleCollisions(bool bShowOn)
{
	// ToggleShowSimpleCollision() does more that just flipping a flag so we allow it to do its thing if needed.
	if (bShowSimpleCollision != bShowOn)
	{
		ToggleShowSimpleCollision();
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowComplexCollision"), bShowPivot ? TEXT("True") : TEXT("False"));
		}
		Invalidate();
	}


	bShowSimpleCollision = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowSimpleCollision"), bShowPivot ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowComplexCollisions(bool bShowOn)
{
	// ToggleShowComplexCollision() does more that just flipping a flag so we allow it to do its thing if needed.
	if (bShowComplexCollision != bShowOn)
	{
		ToggleShowComplexCollision();
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowComplexCollision"), bShowPivot ? TEXT("True") : TEXT("False"));
		}
		Invalidate();
	}
}

void FStaticMeshEditorViewportClient::SetShowPivots(bool bShowOn)
{
	bShowPivot = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowPivot"), bShowPivot ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowGrids(bool bShowOn)
{
 	EngineShowFlags.Grid = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("EngineShowFlags.Grid"), EngineShowFlags.Grid ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

void FStaticMeshEditorViewportClient::SetShowVertices(bool bShowOn)
{
	bDrawVertices = bShowOn;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawVertices"), bDrawVertices ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}


void FStaticMeshEditorViewportClient::SetShowWireframes(bool bShowOn)
{

}


void FStaticMeshEditorViewportClient::SetShowVertexColors(bool bShowOn)
{

}


void FStaticMeshEditorViewportClient::ToggleDrawUVOverlay()
{
	SetDrawUVOverlay(!bDrawUVs);
}

void FStaticMeshEditorViewportClient::SetDrawUVOverlay(bool bShouldDraw)
{
	bDrawUVs = bShouldDraw;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawUVs"), bDrawUVs ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsDrawUVOverlayChecked() const
{
	return bDrawUVs;
}

bool FStaticMeshEditorViewportClient::IsShowNormalsChecked() const
{
	return bDrawNormals;
}

void FStaticMeshEditorViewportClient::ToggleShowTangents()
{
	bDrawTangents = !bDrawTangents;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawTangents"), bDrawTangents ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsShowTangentsChecked() const
{
	return bDrawTangents;
}

void FStaticMeshEditorViewportClient::ToggleShowBinormals()
{
	bDrawBinormals = !bDrawBinormals;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawBinormals"), bDrawBinormals ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsShowBinormalsChecked() const
{
	return bDrawBinormals;
}

void FStaticMeshEditorViewportClient::ToggleDrawVertices()
{
	bDrawVertices = !bDrawVertices;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawVertices"), bDrawVertices ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsDrawVerticesChecked() const
{
	return bDrawVertices;
}

void FStaticMeshEditorViewportClient::ToggleShowSimpleCollision()
{
	bShowSimpleCollision = !bShowSimpleCollision;
	StaticMeshEditorPtr.Pin()->ClearSelectedPrims();
	UpdateSimpleCollisionDisplay();

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowCollision"), (bShowSimpleCollision || bShowComplexCollision) ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditorViewportClient::IsShowSimpleCollisionChecked() const
{
	return bShowSimpleCollision;
}

void FStaticMeshEditorViewportClient::ToggleShowComplexCollision()
{
	bShowComplexCollision = !bShowComplexCollision;
	UpdateComplexCollisionDisplay();
	
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowCollision"), (bShowSimpleCollision || bShowComplexCollision) ? TEXT("True") : TEXT("False"));
	}
}

bool FStaticMeshEditorViewportClient::IsShowComplexCollisionChecked() const
{
	return bShowComplexCollision;
}

void FStaticMeshEditorViewportClient::ToggleShowSockets()
{
	bShowSockets = !bShowSockets;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowSockets"), bShowSockets ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}
bool FStaticMeshEditorViewportClient::IsShowSocketsChecked() const
{
	return bShowSockets;
}

void FStaticMeshEditorViewportClient::ToggleShowPivot()
{
	bShowPivot = !bShowPivot;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bShowPivot"), bShowPivot ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsShowPivotChecked() const
{
	return bShowPivot;
}

void FStaticMeshEditorViewportClient::ToggleDrawAdditionalData()
{
	bDrawAdditionalData = !bDrawAdditionalData;
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("bDrawAdditionalData"), bDrawAdditionalData ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsDrawAdditionalDataChecked() const
{
	return bDrawAdditionalData;
}

TSet< int32 >& FStaticMeshEditorViewportClient::GetSelectedEdges()
{ 
	return SelectedEdgeIndices;
}

void FStaticMeshEditorViewportClient::OnMeshChanged()
{
	UpdateComplexCollisionDisplay();
}

void FStaticMeshEditorViewportClient::OnSocketSelectionChanged( UStaticMeshSocket* SelectedSocket )
{
	if (SelectedSocket)
	{
		SelectedEdgeIndices.Empty();

		if (WidgetMode == UE::Widget::WM_None || WidgetMode == UE::Widget::WM_Scale)
		{
			WidgetMode = UE::Widget::WM_Translate;
		}
	}

	Invalidate();
}

bool FStaticMeshEditorViewportClient::IsCustomModeUsingWidget() const
{
	const UE::Widget::EWidgetMode ToolsWidgetMode = ModeTools->GetWidgetMode();
	const bool bDisplayToolWidget = ModeTools->GetShowWidget();

	return bDisplayToolWidget && ToolsWidgetMode != UE::Widget::EWidgetMode::WM_None;
}
#undef LOCTEXT_NAMESPACE
