// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UnrealWidgetFwd.h"
#include "EditorViewportClient.h"
#include "Components.h"
#include "Interfaces/Interface_CollisionDataProvider.h"

class FAdvancedPreviewScene;
class FCanvas;
class IStaticMeshEditor;
class SStaticMeshEditorViewport;
class UStaticMesh;
class UStaticMeshComponent;
class UStaticMeshSocket;

/** Viewport Client for the preview viewport */
class FStaticMeshEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FStaticMeshEditorViewportClient>
{
public:
	FStaticMeshEditorViewportClient(TWeakPtr<IStaticMeshEditor> InStaticMeshEditor, const TSharedRef<SStaticMeshEditorViewport>& InStaticMeshEditorViewport, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene, UStaticMesh* InPreviewStaticMesh, UStaticMeshComponent* InPreviewStaticMeshComponent);
	~FStaticMeshEditorViewportClient();

	// FEditorViewportClient interface
	virtual void MouseMove(FViewport* Viewport,int32 x, int32 y) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	virtual void ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas ) override;
	virtual bool InputWidgetDelta( FViewport* Viewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale ) override;
	virtual void TrackingStarted( const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge ) override;
	virtual void TrackingStopped() override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override;
	virtual bool CanCycleWidgetMode() const override;
	virtual FVector GetWidgetLocation() const override;
	virtual FMatrix GetWidgetCoordSystem() const override;
	virtual ECoordSystem GetWidgetCoordSystemSpace() const override;
	virtual bool ShouldOrbitCamera() const override;
	
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;

	/** 
	 *	Updates the static mesh and static mesh component being used in the Static Mesh Editor.
	 *
	 *	@param	InStaticMesh				The static mesh handled by the editor.
	 *	@param	InStaticMeshComponent		The static mesh component component from the viewport's scene.
	 */
	void SetPreviewMesh(UStaticMesh* InStaticMesh, UStaticMeshComponent* InPreviewStaticMeshComponent, bool bResetCamera=true);

	/** Retrieves the selected edge set. */
	TSet< int32 >& GetSelectedEdges();

	/** Called when the mesh changes **/
	void OnMeshChanged();

	/**
	 * Called when the selected socket changes
	 *
	 * @param SelectedSocket	The newly selected socket or NULL if no socket is selected
	 */
	void OnSocketSelectionChanged( UStaticMeshSocket* SelectedSocket );

	/**
	 *	Draws the UV overlay for the current LOD.
	 *
	 *	@param	InViewport					The viewport to draw to
	 *	@param	InCanvas					The canvas to draw to
	 *	@param	InTextYPos					The Y-position to begin drawing the UV-Overlay at (due to text displayed above it).
	 */
	void DrawUVsForMesh(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos );

	/** Callback for toggling the UV overlay show flag. */
	void ToggleDrawUVOverlay();

	void SetDrawUVOverlay(bool bShouldDraw);

	/** Callback for checking the UV overlay show flag. */
	bool IsDrawUVOverlayChecked() const;


	/** Set the normals show flag. */
	void SetShowNormals(bool bShowOn);

	/** Set the tangents show flag. */
	void SetShowTangents(bool bShowOn);

	/** Set the binormals show flag. */
	void SetShowBinormals(bool bShowOn);

	/** Set the simple collisions show flag. */
	void SetShowSimpleCollisions(bool bShowOn);

	/** Set the complex collisions show flag. */
	void SetShowComplexCollisions(bool bShowOn);

	/** Set the pivots show flag. */
	void SetShowPivots(bool bShowOn);

	/** Set the grids show flag. */
	void SetShowGrids(bool bShowOn);

	/** Set the vertices show flag. */
	void SetShowVertices(bool bShowOn);

	/** Set the wireframes show flag. */
	void SetShowWireframes(bool bShowOn);

	/** Set the vertex colors show flag. */
	void SetShowVertexColors(bool bShowOn);

	/** Callback for toggling the normals show flag. */
	void ToggleShowNormals();
	
	/** Callback for checking the normals show flag. */
	bool IsShowNormalsChecked() const;

	/** Callback for toggling the tangents show flag. */
	void ToggleShowTangents();
	
	/** Callback for checking the tangents show flag. */
	bool IsShowTangentsChecked() const;

	/** Callback for toggling the binormals show flag. */
	void ToggleShowBinormals();
	
	/** Callback for checking the binormals show flag. */
	bool IsShowBinormalsChecked() const;

	/** Callback for toggling simple collision drawing. */
	void ToggleShowSimpleCollision();

	/** Callback for checking simple collision drawing. */
	bool IsShowSimpleCollisionChecked() const;

	/** Callback for toggling complex collision drawing. */
	void ToggleShowComplexCollision();

	/** Callback for checking complex collision drawing. */
	bool IsShowComplexCollisionChecked() const;

	/** Callback for toggling the socket show flag. */
	void ToggleShowSockets();

	/** Callback for checking the socket show flag. */
	bool IsShowSocketsChecked() const;

	/** Callback for toggling the pivot show flag. */
	void ToggleShowPivot();

	/** Callback for checking the pivot show flag. */
	bool IsShowPivotChecked() const;

	/** Callback for toggling the additional data drawing flag. */
	void ToggleDrawAdditionalData();

	/** Callback for checking the additional data drawing flag. */
	bool IsDrawAdditionalDataChecked() const;

	/** Callback for toggling the vertices drawing flag. */
	void ToggleDrawVertices();

	/** Callback for checking the vertices drawing flag. */
	bool IsDrawVerticesChecked() const;

	/** Used to toggle the floor when vertex colours should be shown */
	void SetFloorAndEnvironmentVisibility(const bool bVisible);
protected:
	// FEditorViewportClient interface
	virtual void PerspectiveCameraMoved() override;

	/** Used to update the display of simple collision **/
	void UpdateSimpleCollisionDisplay();
	/** Used to update the display of complex collision **/
	void UpdateComplexCollisionDisplay();
private:
	bool IsCustomModeUsingWidget() const;

	void BeginManipulation(bool bDuplicate);

	/** The Simplygon logo to be drawn when Simplygon has been used on the static mesh. */
	UTexture2D* SimplygonLogo;

	/** Component for the static mesh. */
	UStaticMeshComponent* StaticMeshComponent;

	/** The static mesh being used in the editor. */
	UStaticMesh* StaticMesh;

	FTriMeshCollisionData CollisionMeshData;

	/** Pointer back to the StaticMesh editor tool that owns us */
	TWeakPtr<IStaticMeshEditor> StaticMeshEditorPtr;

	/** Flags for various options in the editor. */
	bool bDrawUVs;
	bool bShowSimpleCollision;
	bool bShowComplexCollision;
	bool bShowSockets;
	bool bDrawNormals;
	bool bDrawTangents;
	bool bDrawBinormals;
	bool bShowPivot;
	bool bDrawAdditionalData;
	bool bDrawVertices;

	/** true when the user is manipulating a socket widget. */
	bool bManipulating;

	UE::Widget::EWidgetMode WidgetMode;

	/** The current widget axis the mouse is highlighting. */
	EAxis::Type SocketManipulateAxis;

	/** Holds the currently selected edges. */
	typedef TSet< int32 > FSelectedEdgeSet;
	FSelectedEdgeSet SelectedEdgeIndices;

	/** Cached vertex positions for the currently selected edges. Used for rendering */
	TArray<FVector> SelectedEdgeVertices;

	/** Cached tex coords for the currently selected edges. Used for rendering UV coordinates*/
	TArray<FVector2D> SelectedEdgeTexCoords[MAX_STATIC_TEXCOORDS];

	/** Pointer back to the StaticMeshEditor viewport control that owns us */
	TWeakPtr<SStaticMeshEditorViewport> StaticMeshEditorViewportPtr;

	/** Stored pointer to the preview scene in which the static mesh is shown */
	FAdvancedPreviewScene* AdvancedPreviewScene;
};
