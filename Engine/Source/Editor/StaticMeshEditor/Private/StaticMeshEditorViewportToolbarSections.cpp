// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorViewportToolbarSections.h"

#include "SStaticMeshEditorViewport.h"
#include "StaticMeshEditorActions.h"
#include "StaticMeshViewportLODCommands.h"
#include "RayTracingDebugVisualizationMenuCommands.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditorViewportToolbarSections"

FText UE::StaticMeshEditor::GetLODMenuLabel(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport)
{
	FText Label = LOCTEXT("LODMenu_AutoLabel", "LOD Auto");

	if (InStaticMeshEditorViewport)
	{
		const int32 LODSelectionType = InStaticMeshEditorViewport->GetCurrentLOD();

		if (LODSelectionType >= 0)
		{
			FString TitleLabel = FString::Printf(TEXT("LOD %d"), LODSelectionType);
			Label = FText::FromString(TitleLabel);
		}
	}

	return Label;
}

FToolMenuEntry UE::StaticMeshEditor::CreateLODSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitDynamicEntry(
		"DynamicLODOptions",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				if (UUnrealEdViewportToolbarContext* const EditorViewportContext = InDynamicSection.FindContext<UUnrealEdViewportToolbarContext>())
				{
					TWeakPtr<SStaticMeshEditorViewport> StaticMeshEditorViewport = StaticCastWeakPtr<SStaticMeshEditorViewport>(EditorViewportContext->Viewport);
					FToolMenuEntry& Entry = InDynamicSection.AddEntry(UE::UnrealEd::CreatePreviewLODSelectionSubmenu(StaticMeshEditorViewport));
					Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
				}
			}
		)
	);
	return Entry;
}

FToolMenuEntry UE::StaticMeshEditor::CreateShowSubmenu()
{
	return UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateLambda(&FillShowSubmenu));
}

FToolMenuEntry UE::StaticMeshEditor::CreateAspectRatioSubmenu()
{
	const FNewToolMenuChoice InSubmenuChoice = FNewToolMenuDelegate::CreateLambda(&FillAspectRatioSubmenu);

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"Aspect Ratio",
		LOCTEXT("AspectSubmenuLabel", "Aspect Ratio"),
		LOCTEXT("AspectSubmenuTooltip", "Control the aspect ratio of the current viewport"),
		InSubmenuChoice
	);

	Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.ToggleAllowConstrainedAspectRatioInPreview");
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.ClippingPriority = 800;

	return Entry;
}


TSharedRef<SWidget> UE::StaticMeshEditor::GenerateLODMenuWidget(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport
)
{
	if (!InStaticMeshEditorViewport)
	{
		return SNullWidget::NullWidget;
	}

	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "StaticMeshEditor.OldViewportToolbar.LODMenu";

	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false))
		{
			Menu->AddDynamicSection(
				"BaseSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						if (UUnrealEdViewportToolbarContext* const EditorViewportContext = InMenu->FindContext<UUnrealEdViewportToolbarContext>())
						{
							UE::UnrealEd::FillPreviewLODSelectionSubmenu(InMenu, StaticCastWeakPtr<SStaticMeshEditorViewport>(EditorViewportContext->Viewport));
						}
					}
				)
			);
		}
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InStaticMeshEditorViewport->GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(InStaticMeshEditorViewport);

			MenuContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

TSharedRef<SWidget> UE::StaticMeshEditor::GenerateShowMenuWidget(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport
)
{
	if (!InStaticMeshEditorViewport)
	{
		return SNullWidget::NullWidget;
	}

	InStaticMeshEditorViewport->OnFloatingButtonClicked();

	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "StaticMesh.OldViewportToolbar.Show";

	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false);
		Menu->AddDynamicSection(
			"BaseSection",
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					constexpr bool bShowGridToggle = true;
					FillShowSubmenu(InMenu);
				}
			)
		);
	}

	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InStaticMeshEditorViewport->GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(InStaticMeshEditorViewport);

			MenuContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

void UE::StaticMeshEditor::FillShowSubmenu(UToolMenu* InMenu)
{
	if (UUnrealEdViewportToolbarContext* const EditorViewportContext =
			InMenu->FindContext<UUnrealEdViewportToolbarContext>())
	{
		if (TSharedPtr<SStaticMeshEditorViewport> StaticMeshEditorViewport =
				StaticCastSharedPtr<SStaticMeshEditorViewport>(EditorViewportContext->Viewport.Pin()))
		{
			FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);
			const FStaticMeshEditorCommands& Commands = FStaticMeshEditorCommands::Get();

			UnnamedSection.AddMenuEntry(Commands.SetShowNaniteFallback);
			UnnamedSection.AddMenuEntry(Commands.SetShowDistanceField);
			UnnamedSection.AddMenuEntry(Commands.SetShowRayTracingFallback);

			FToolMenuSection& MeshComponentsSection =
				InMenu->FindOrAddSection("MeshComponents", LOCTEXT("MeshComponments", "Mesh Components"));

			MeshComponentsSection.AddMenuEntry(Commands.SetShowSockets);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowVertices);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowVertexColor);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowNormals);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowTangents);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowBinormals);

			MeshComponentsSection.AddSeparator(NAME_None);

			MeshComponentsSection.AddMenuEntry(Commands.SetShowPivot);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowGrid);

			MeshComponentsSection.AddMenuEntry(Commands.SetShowBounds);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowSimpleCollision);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowComplexCollision);
			MeshComponentsSection.AddMenuEntry(Commands.SetShowPhysicalMaterialMasks);
		}
	}
}

void UE::StaticMeshEditor::FillAspectRatioSubmenu(UToolMenu* InMenu)
{
	if (UUnrealEdViewportToolbarContext* const EditorViewportContext =
		InMenu->FindContext<UUnrealEdViewportToolbarContext>())
	{
		if (TSharedPtr<SStaticMeshEditorViewport> StaticMeshEditorViewport =
			StaticCastSharedPtr<SStaticMeshEditorViewport>(EditorViewportContext->Viewport.Pin()))
		{
			FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);
			const FStaticMeshEditorCommands& Commands = FStaticMeshEditorCommands::Get();

			UnnamedSection.AddMenuEntry(Commands.SetAspectRatio_Free);
			UnnamedSection.AddMenuEntry(Commands.SetAspectRatio_Platform);

			FToolMenuSection& MobileAspectRatiosSection =
				InMenu->FindOrAddSection("MobileAspectRatios", LOCTEXT("MobileAspectRatios", "Mobile Aspect Ratios"));

			MobileAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_6_13);
			MobileAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_13_6);

			FToolMenuSection& DesktopAspectRatiosSection =
				InMenu->FindOrAddSection("DesktopAspectRatios", LOCTEXT("DesktopAspectRatios", "Desktop Aspect Ratios"));

			DesktopAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_1_1);
			DesktopAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_4_3);
			DesktopAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_16_10);
			DesktopAspectRatiosSection.AddMenuEntry(Commands.SetAspectRatio_16_9);
		}
	}
}

void UE::StaticMeshEditor::ExtendViewModesSubmenu(FName InViewModesSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InViewModesSubmenuName);

	Submenu->AddDynamicSection(
		"LevelEditorViewModesExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				if (UUnrealEdViewportToolbarContext* const EditorViewportContext =
					InDynamicMenu->FindContext<UUnrealEdViewportToolbarContext>())
				{
					if (TSharedPtr<SStaticMeshEditorViewport> StaticMeshEditorViewport =
						StaticCastSharedPtr<SStaticMeshEditorViewport>(EditorViewportContext->Viewport.Pin()))
					{
						FToolMenuSection& Section = InDynamicMenu->FindOrAddSection("ViewMode");
						Section.AddSubMenu(
							"RayTracingDebugSubMenu",
							LOCTEXT("RayTracingDebugSubMenu", "Ray Tracing Debug"),
							LOCTEXT("RayTracing_ToolTip", "Select ray tracing buffer visualization view modes"),
							FNewToolMenuDelegate::CreateStatic(&FRayTracingDebugVisualizationMenuCommands::BuildVisualisationSubMenu),
							FUIAction(
								FExecuteAction(),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda(
									[WeakViewport = StaticMeshEditorViewport.ToWeakPtr()]()
									{
										const TSharedPtr<SStaticMeshEditorViewport> Viewport = WeakViewport.Pin();
										check(Viewport.IsValid());
										FEditorViewportClient& ViewportClient = Viewport->GetViewportClient();
										return ViewportClient.IsViewModeEnabled(VMI_RayTracingDebug);
									}
								)
							),
							EUserInterfaceActionType::RadioButton,
							/* bInOpenSubMenuOnClick = */ false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RayTracingDebugMode")
						);
					}
				}
			}
		)
	);
}


#undef LOCTEXT_NAMESPACE
