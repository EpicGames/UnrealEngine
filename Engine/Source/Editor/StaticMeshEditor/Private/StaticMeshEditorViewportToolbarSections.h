// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "ToolMenuEntry.h"

class SStaticMeshEditorViewport;
class SWidget;

namespace UE::StaticMeshEditor
{
FToolMenuEntry CreateLODSubmenu();
FToolMenuEntry CreateShowSubmenu();
FToolMenuEntry CreateAspectRatioSubmenu();

TSharedRef<SWidget> GenerateLODMenuWidget(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport);
TSharedRef<SWidget> GenerateShowMenuWidget(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport);

void FillShowSubmenu(UToolMenu *InMenu);
void FillAspectRatioSubmenu(UToolMenu* InMenu);

FText GetLODMenuLabel(const TSharedPtr<SStaticMeshEditorViewport>& InStaticMeshEditorViewport);

void ExtendViewModesSubmenu(FName InViewModesSubmenuName);
}
