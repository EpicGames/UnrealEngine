// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshViewportLODCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "StaticMeshViewportLODCommands"

void FStaticMeshViewportLODCommands::RegisterCommands()
{
	UI_COMMAND( LODAuto, "LOD Auto", "Automatically select LOD", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LOD0, "LOD 0", "Force select LOD 0", EUserInterfaceActionType::RadioButton, FInputChord() );
}

#undef LOCTEXT_NAMESPACE
