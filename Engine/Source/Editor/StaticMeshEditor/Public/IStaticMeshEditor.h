// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Misc/MessageDialog.h"
#include "PhysicsEngine/ShapeElem.h"

class UStaticMesh;
class UStaticMeshComponent;
class UStaticMeshSocket;


DECLARE_MULTICAST_DELEGATE(FOnSelectedLODChangedMulticaster);

typedef FOnSelectedLODChangedMulticaster::FDelegate FOnSelectedLODChanged;

/**
 * Public interface to Static Mesh Editor
 */
class IStaticMeshEditor : public FAssetEditorToolkit
{
public:
	/**
	 * Primitive data use to track which aggregate geometry is selected
	 */
	struct FPrimData
	{
		EAggCollisionShape::Type		PrimType;
		int32							PrimIndex;

		FPrimData(EAggCollisionShape::Type InPrimType, int32 InPrimIndex) :
			PrimType(InPrimType),
			PrimIndex(InPrimIndex) {
		}

		bool operator==(const FPrimData& Other) const
		{
			return (PrimType == Other.PrimType && PrimIndex == Other.PrimIndex);
		}
	};

	/** Called after an undo is performed to give child widgets a chance to refresh. */
	DECLARE_MULTICAST_DELEGATE(FOnPostUndoMulticaster);

	// Post undo
	typedef FOnPostUndoMulticaster::FDelegate FOnPostUndo;

	/** Retrieves the current static mesh displayed in the Static Mesh Editor. */
	virtual UStaticMesh* GetStaticMesh() = 0;

	/** Retrieves the static mesh component. */
	virtual UStaticMeshComponent* GetStaticMeshComponent() const = 0;

	/** Retrieves the first currently selected socket from the Socket Manager. */
	UE_DEPRECATED(5.6, "GetSelectedSocket is deprecated, use GetSelectedSockets instead.")
		virtual UStaticMeshSocket* GetSelectedSocket() const = 0;

	/** Retrieves the currently selected sockets from the Socket Manager. */
	virtual TArray<UStaticMeshSocket*> GetSelectedSockets() const = 0;

	/** Returns true if at least one socket is selected. */
	virtual bool HasSelectedSockets() const = 0;

	/**
	 *	Set the currently selected socket in the Socket Manager.
	 *
	 *	@param	InSelectedSocket			The selected socket to pass on to the Socket Manager.
	 */
	virtual void SetSelectedSocket(UStaticMeshSocket* InSelectedSocket) = 0;

	/**
	 *	Adds a socket to the currently selected sockets in the Socket Manager.
	 *
	 *	@param	InSelectedSocket		The socket to add to the current selection in the Socket Manager.
	 */
	virtual void AddSelectedSocket(UStaticMeshSocket* InSelectedSocket) = 0;

	/**
	*	Removes a socket form the currently selected sockets in the Socket Manager.
	*
	*	@param	InSelectedSocket		The socket to removed from the current selection in the Socket Manager.
	*/
	virtual void RemoveSelectedSocket(const UStaticMeshSocket* InSelectedSocket) = 0;

	/** Duplicate the first selected socket */
	UE_DEPRECATED(5.6, "DuplicateSelectedSocket is deprecated, use DuplicateSelectedSockets instead.")
		virtual void DuplicateSelectedSocket() = 0;

	/** Duplicate the selected sockets */
	virtual void DuplicateSelectedSockets() = 0;

	/** Requests to rename selected socket */
	virtual void RequestRenameSelectedSocket() = 0;

	/**
	 *  Checks to see if the prim data is valid compared with the static mesh
	 *
	 *  @param  InPrimData			The data to check
	 */
	virtual bool IsPrimValid(const FPrimData& InPrimData) const = 0;

	/** Checks to see if any prims are selected */
	virtual bool HasSelectedPrims() const = 0;

	/** Get selected prims */
	virtual const TArray<FPrimData>& GetSelectedPrims() const = 0;

	/**
	 *  Adds primitive information to the selected prims list
	 *
	 *  @param  InPrimData			The data to add
	 *  @param  bClearSelection		If true, clears the current selection
	 */
	virtual void AddSelectedPrim(const FPrimData& InPrimData, bool bClearSelection) = 0;

	/**
	 *  Removes primitive information to the selected prims list
	 *
	 *  @param  InPrimData			The data to remove
	 */
	virtual void RemoveSelectedPrim(const FPrimData& InPrimData) = 0;

	/** Removes all invalid primitives from the list */
	virtual void RemoveInvalidPrims() = 0;

	/**
	 *  Checks to see if the parsed primitive data is selected
	 *
	 *  @param  InPrimData			The data to compare
	 *  @returns					True, if the prim is selected
	 */
	virtual bool IsSelectedPrim(const FPrimData& InPrimData) const = 0;

	/** Removes all primitive data from the list */
	virtual void ClearSelectedPrims() = 0;

	/**
	 *  Duplicates all the selected primitives and selects them
	 *
	 *  @param  InOffset			[optional] Allows the duplicate to be offset by this factor
	 */
	virtual void DuplicateSelectedPrims(const FVector* InOffset) = 0;

	/**
	 *  Copy all the selected primitives to clipboard
	 */
	virtual int32 CopySelectedPrims() const = 0;

	/**
	 *  Paste all the selected primitives from clipboard
	 */
	virtual int32 PasteCopiedPrims() = 0;

	/**
	 *  Translates the selected primitives by the specified amount
	 *
	 *  @param  InDrag				The amount to translate
	 */
	virtual void TranslateSelectedPrims(const FVector& InDrag) = 0;

	/**
	 *  Rotates the selected primitives by the specified amount
	 *
	 *  @param  InRot				The amount to rotate
	 */
	virtual void RotateSelectedPrims(const FRotator& InRot) = 0;

	/**
	 *  Scales the selected primitives by the specified amount
	 *
	 *  @param  InScale				The amount to scale
	 */
	virtual void ScaleSelectedPrims(const FVector& InScale) = 0;

	/**
	 *  Calculates the bounding box of our selected primitives
	 *
	 *  @param  OutBox				The bounding data for our selection
	 *  @returns					True, if there are any prims selected
	 */
	virtual bool CalcSelectedPrimsAABB(FBox& OutBox) const = 0;

	/**
	 *  Fetches the transform of the last primitive to be selected
	 *
	 *  @param  OutTransform		The transform of the last selected primitive
	 *  @returns					True, if there was a prim selected
	 */
	virtual bool GetLastSelectedPrimTransform(FTransform& OutTransform) const = 0;

	/**
	 *  Gets the transform of the specified primitive
	 *
	 *  @param  InPrimData			The data about the primitive
	 *  @returns					The transform of the specified primitive
	 */
	virtual FTransform GetPrimTransform(const FPrimData& InPrimData) const = 0;

	/**
	 *  Sets the transform of the specified primitive
	 *
	 *  @param  InPrimData			The data about the primitive
	 *  @param  InPrimTransform		The transform to apply to the primitive
	 */
	virtual void SetPrimTransform(const FPrimData& InPrimData, const FTransform& InPrimTransform) const = 0;

	/**
	 *  Retrieves the number of triangles in the current static mesh or it's forced LOD.
	 *
	 *  @param  LODLevel			The desired LOD to retrieve the number of triangles for.
	 *	@returns					The number of triangles for the specified LOD level.
	 */
	virtual int32 GetNumTriangles(int32 LODLevel = 0) const = 0;

	/**
	 *  Retrieves the number of vertices in the current static mesh or it's forced LOD.
	 *
	 *  @param  LODLevel			The desired LOD to retrieve the number of vertices for.
	 *	@returns					The number of vertices for the specified LOD level.
	 */
	virtual int32 GetNumVertices(int32 LODLevel = 0) const = 0;

	/**
	 *  Retrieves the number of UV channels available.
	 *
	 *  @param  LODLevel			The desired LOD to retrieve the number of UV channels for.
	 *	@returns					The number of triangles for the specified LOD level.
	 */
	virtual int32 GetNumUVChannels(int32 LODLevel = 0) const = 0;

	/** Retrieves the currently selected UV channel. */
	virtual int32 GetCurrentUVChannel() = 0;

	/** Retrieves the current LOD level. 0 is auto, 1 is base. */
	virtual int32 GetCurrentLODLevel() = 0;

	/** Retrieves the current LOD index */
	virtual int32 GetCurrentLODIndex() = 0;

	/* Retrieve editor custom data. Return INDEX_NONE if the key is invalid */
	virtual int32 GetCustomData(const int32 Key) const { return INDEX_NONE; }

	/*
	 * Store the custom data using the key.
	 * Remark:
	 * The custom data memory should be clear when the editor is close by the user, this is not persistent data.
	 * Currently we use it to store the state of the editor UI to restore it properly when a refresh happen.
	 */
	virtual void SetCustomData(const int32 Key, const int32 CustomData) {}

	/** Refreshes the Static Mesh Editor's viewport. */
	virtual void RefreshViewport() = 0;

	/** Returns whether simple collision is currently shown in the viewport. */
	virtual bool IsShowSimpleCollisionChecked() const = 0;

	/** Toggles simple collision visibility in the viewport. */
	virtual void ToggleShowSimpleCollision() = 0;

	/** Refreshes everything in the Static Mesh Editor. */
	virtual void RefreshTool() = 0;

	/**
	*    This is called when Apply is pressed in the dialog. Does the actual processing.
	*
	*    @param    InHullCount               The max number of convex hulls allowed
	*    @param    InMaxHullVerts            The max number of verts per hull allowed.
	*    @param    InHullPrecision           The voxel precision to use for V-HACD
	*/
	virtual void DoDecomp(uint32 InHullCount, int32 InMaxHullVerts, uint32 InHullPrecision) = 0;


	/** Retrieves the selected edge set. */
	virtual TSet< int32 >& GetSelectedEdges() = 0;

	/** Registers a delegate to be called after an Undo operation */
	virtual void RegisterOnPostUndo(const FOnPostUndo& Delegate) = 0;

	/** Unregisters a delegate to be called after an Undo operation */
	virtual void UnregisterOnPostUndo(SWidget* Widget) = 0;

	/** Get the active view mode */
	virtual EViewModeIndex GetViewMode() const = 0;

	/* Register callback to be able to be notify when the select LOD is change */
	virtual void RegisterOnSelectedLODChanged(const FOnSelectedLODChanged& Delegate, bool UnregisterOnRefresh) = 0;
	/* Unregister callback to free up the ressources */
	virtual void UnRegisterOnSelectedLODChanged(FDelegateUserObject Thing) = 0;

	/* Broadcast when selected LOD changes */
	virtual void BroadcastOnSelectedLODChanged() = 0;

	/** Get the Static Mesh Editor's the viewport client */
	virtual class FEditorViewportClient& GetViewportClient() = 0;

	/** Set the display name for the secondary toolbar and its associated menu item */
	virtual void SetSecondaryToolbarDisplayName(FText DisplayName) = 0;

	DECLARE_EVENT_OneParam(IStaticMeshEditor, FStaticMeshEditorDockingExtentionTabsEvent, const TSharedRef<FTabManager::FStack>&)
	virtual FStaticMeshEditorDockingExtentionTabsEvent& OnStaticMeshEditorDockingExtentionTabs() { return StaticMeshEditorDockingExtentionTabsEvent; };

	DECLARE_EVENT(IStaticMeshEditor, FStaticMeshEditorClosedEvent);
	virtual FStaticMeshEditorClosedEvent& OnStaticMeshEditorClosed() { return StaticMeshEditorClosedEvent; };

	/** Delegate to be called when the tabs are being registered **/
	DECLARE_EVENT_OneParam(IStaticMeshEditor, FRegisterTabSpawnersEvent, const TSharedRef<class FTabManager>&);
	virtual FRegisterTabSpawnersEvent& OnRegisterTabSpawners() { return RegisterTabSpawnersEvent; };

	/** Delegate to be called when the tabs are being unregistered **/
	DECLARE_EVENT_OneParam(IStaticMeshEditor, FUnregisterTabSpawnersEvent, const TSharedRef<class FTabManager>&);
	virtual FUnregisterTabSpawnersEvent& OnUnregisterTabSpawners() { return UnregisterTabSpawnersEvent; };

	DECLARE_DELEGATE_RetVal_OneParam(bool, FAddSaveAssetValidator, FString&);
	virtual FDelegateHandle AddSaveAssetValidator(const FAddSaveAssetValidator& Validator)
	{
		check(Validator.IsBound());
		const FDelegateHandle ValidatorHandle = Validator.GetHandle();
		SaveAssetValidators.Add(ValidatorHandle, Validator);
		return ValidatorHandle;
	};
	virtual bool RemoveSaveAssetValidator(const FDelegateHandle& ValidatorHandle)
	{
		return SaveAssetValidators.Remove(ValidatorHandle) > 0;
	};

protected:

	virtual void SaveAsset_Execute() override
	{
		TArray<FDelegateHandle> ValidatorHandles;
		SaveAssetValidators.GetKeys(ValidatorHandles);

		for (const FDelegateHandle& Handle : ValidatorHandles)
		{
			const FAddSaveAssetValidator* Validator = SaveAssetValidators.Find(Handle);
			if (Validator && Validator->IsBound())
			{
				FString Err;
				if (!Validator->Execute(Err))
				{
					FMessageDialog::Open(
						EAppMsgType::Ok,
						FText::FromString(Err)
					);
					return;
				}
			}
		}

		FAssetEditorToolkit::SaveAsset_Execute();
	};

private:
	FStaticMeshEditorDockingExtentionTabsEvent StaticMeshEditorDockingExtentionTabsEvent;
	FStaticMeshEditorClosedEvent StaticMeshEditorClosedEvent;
	FRegisterTabSpawnersEvent RegisterTabSpawnersEvent;
	FUnregisterTabSpawnersEvent UnregisterTabSpawnersEvent;

	TMap<FDelegateHandle, FAddSaveAssetValidator> SaveAssetValidators;
};


