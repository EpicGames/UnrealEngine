// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

/**
 * Unreal StaticMesh editor actions
 */
class FStaticMeshEditorCommands : public TCommands<FStaticMeshEditorCommands>
{

public:
	FStaticMeshEditorCommands() : TCommands<FStaticMeshEditorCommands>
		(
			"StaticMeshEditor", // Context name for fast lookup
			NSLOCTEXT("Contexts", "StaticMeshEditor", "StaticMesh Editor"), // Localized context name for displaying
			"EditorViewport",  // Parent
			FAppStyle::GetAppStyleSetName() // Icon Style Set
			)
	{
	}

	/**
	 * StaticMesh Editor Commands
	 */

	 /**  */
	TSharedPtr<FUICommandInfo> SetShowNaniteFallback;
	TSharedPtr<FUICommandInfo> SetShowDistanceField;
	TSharedPtr<FUICommandInfo> SetShowRayTracingFallback;
	TSharedPtr<FUICommandInfo> SetShowWireframe;
	TSharedPtr<FUICommandInfo> SetShowVertexColor;
	TSharedPtr<FUICommandInfo> SetShowPhysicalMaterialMasks;
	TSharedPtr<FUICommandInfo> SetDrawUVs;
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowSimpleCollision;
	TSharedPtr<FUICommandInfo> SetShowComplexCollision;
	TSharedPtr<FUICommandInfo> ResetCamera;
	TSharedPtr<FUICommandInfo> SetShowSockets;
	TSharedPtr<FUICommandInfo> SetDrawAdditionalData;

	// Mesh toolbar Commands
	TSharedPtr<FUICommandInfo> ReimportMesh;
	TSharedPtr<FUICommandInfo> ReimportMeshWithNewFile;
	TSharedPtr<FUICommandInfo> ReimportAllMesh;
	TSharedPtr<FUICommandInfo> ReimportAllMeshWithNewFile;
	TSharedPtr<FUICommandInfo> ReimportMeshWithDialog;
	TSharedPtr<FUICommandInfo> ReimportMeshWithNewFileWithDialog;
	TSharedPtr<FUICommandInfo> ReimportAllMeshWithDialog;
	TSharedPtr<FUICommandInfo> ReimportAllMeshWithNewFileWithDialog;

	// toolbar commands
	TSharedPtr<FUICommandInfo> ToggleShowNormals;
	TSharedPtr<FUICommandInfo> ToggleShowTangents;
	TSharedPtr<FUICommandInfo> ToggleShowBinormals;
	TSharedPtr<FUICommandInfo> ToggleShowPivots;
	TSharedPtr<FUICommandInfo> ToggleShowVertices;
	TSharedPtr<FUICommandInfo> ToggleShowGrids;
	TSharedPtr<FUICommandInfo> ToggleShowBounds;
	TSharedPtr<FUICommandInfo> ToggleShowSimpleCollisions;
	TSharedPtr<FUICommandInfo> ToggleShowComplexCollisions;
	TSharedPtr<FUICommandInfo> ToggleShowSockets;
	TSharedPtr<FUICommandInfo> ToggleShowWireframes;
	TSharedPtr<FUICommandInfo> ToggleShowVertexColors;

	// View Menu Commands
	TSharedPtr<FUICommandInfo> SetShowNormals;
	TSharedPtr<FUICommandInfo> SetShowTangents;
	TSharedPtr<FUICommandInfo> SetShowBinormals;
	TSharedPtr<FUICommandInfo> SetShowPivot;
	TSharedPtr<FUICommandInfo> SetShowVertices;

	// Collision Menu Commands
	TSharedPtr<FUICommandInfo> CreateDOP10X;
	TSharedPtr<FUICommandInfo> CreateDOP10Y;
	TSharedPtr<FUICommandInfo> CreateDOP10Z;
	TSharedPtr<FUICommandInfo> CreateDOP18;
	TSharedPtr<FUICommandInfo> CreateDOP26;
	TSharedPtr<FUICommandInfo> CreateBoxCollision;
	TSharedPtr<FUICommandInfo> CreateSphereCollision;
	TSharedPtr<FUICommandInfo> CreateSphylCollision;
	TSharedPtr<FUICommandInfo> CreateAutoConvexCollision;
	TSharedPtr<FUICommandInfo> RemoveCollision;
	TSharedPtr<FUICommandInfo> ConvertBoxesToConvex;
	TSharedPtr<FUICommandInfo> CopyCollisionFromSelectedMesh;

	// Mesh Menu Commands
	TSharedPtr<FUICommandInfo> FindSource;

	TSharedPtr<FUICommandInfo> ChangeMesh;

	TSharedPtr<FUICommandInfo> BakeMaterials;

	TSharedPtr<FUICommandInfo> SaveGeneratedLODs;

	// Viewport Aspect Ratio Commands

	TSharedPtr<FUICommandInfo> SetAspectRatio_Free;
	TSharedPtr<FUICommandInfo> SetAspectRatio_Platform;
	TSharedPtr<FUICommandInfo> SetAspectRatio_6_13;
	TSharedPtr<FUICommandInfo> SetAspectRatio_13_6;
	TSharedPtr<FUICommandInfo> SetAspectRatio_1_1;
	TSharedPtr<FUICommandInfo> SetAspectRatio_4_3;
	TSharedPtr<FUICommandInfo> SetAspectRatio_16_10;
	TSharedPtr<FUICommandInfo> SetAspectRatio_16_9;
	
	 /**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

public:
};
