// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

class IStaticMeshEditor;
class UStaticMesh;

extern const FName StaticMeshEditorAppIdentifier;

/**
 * Static mesh editor module interface
 */
class IStaticMeshEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/**
	 * Creates a new static mesh editor.
	 */
	virtual TSharedRef<IStaticMeshEditor> CreateStaticMeshEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UStaticMesh* StaticMesh ) = 0;

	/** Delegate to be called when a Material Editor is created, for toolbar, tab, and menu extension **/
	DECLARE_EVENT_OneParam(IStaticMeshEditorModule, FStaticMeshEditorOpenedEvent, TWeakPtr<IStaticMeshEditor>);
	virtual FStaticMeshEditorOpenedEvent& OnStaticMeshEditorOpened() { return StaticMeshEditorOpenedEvent; };

	virtual TSharedPtr<FExtensibilityManager> GetSecondaryToolBarExtensibilityManager() = 0;

	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FStaticMeshEditorToolbarExtender, const TSharedRef<FUICommandList>, TSharedRef<IStaticMeshEditor>);
	virtual TArray<FStaticMeshEditorToolbarExtender>& GetAllStaticMeshEditorToolbarExtenders() = 0;

	DECLARE_EVENT_OneParam(IStaticMeshEditor, FOnRegisterLayoutExtensions, FLayoutExtender&);
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() = 0;

private:
	FStaticMeshEditorOpenedEvent StaticMeshEditorOpenedEvent;
};
