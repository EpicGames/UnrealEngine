// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EditorSubsystem.h"

#include "Engine/StaticMesh.h"
#include "MeshReductionSettings.h"
#include "GameFramework/Actor.h"
#include "BodySetupEnums.h"
#include "UVMapSettings.h"
#include "StaticMeshEditorSubsystemHelpers.h"

#include "StaticMeshEditorSubsystem.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogStaticMeshEditorSubsystem, Log, All);

/**
* UStaticMeshEditorSubsystem
* Subsystem for exposing static mesh functionality to scripts
*/
UCLASS(MinimalAPI)
class UStaticMeshEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	STATICMESHEDITOR_API UStaticMeshEditorSubsystem();

	/**
	 * Remove then add LODs on a static mesh.
	 * The static mesh must have at least LOD 0.
	 * The LOD 0 of the static mesh is kept after removal.
	 * The build settings of LOD 0 will be applied to all subsequent LODs.
	 * @param	StaticMesh				Mesh to process.
	 * @param	ReductionOptions		Options on how to generate LODs on the mesh.
	 * @param	bApplyChanges				Indicates if change must be notified.
	 * @return the number of LODs generated on the input mesh.
	 * An negative value indicates that the reduction could not be performed. See log for explanation.
	 * No action will be performed if ReductionOptions.ReductionSettings is empty
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 SetLodsWithNotification(UStaticMesh* StaticMesh, const FStaticMeshReductionOptions& ReductionOptions, bool bApplyChanges);

	/**
	 * Same as SetLodsWithNotification but changes are applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	int32 SetLods(UStaticMesh* StaticMesh, const FStaticMeshReductionOptions& ReductionOptions)
	{
		return SetLodsWithNotification(StaticMesh, ReductionOptions, true);
	}

	/**
	 * Copy the reduction options with the specified LOD reduction settings.
	 * @param StaticMesh - Mesh to process.
	 * @param LodIndex - The LOD we get the reduction settings.
	 * @param OutReductionOptions - The reduction settings where we copy the reduction options.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API void GetLodReductionSettings(const UStaticMesh* StaticMesh, const int32 LodIndex, FMeshReductionSettings& OutReductionOptions);

	/**
	 * Set the LOD reduction for the specified LOD index.
	 * @param StaticMesh - Mesh to process.
	 * @param LodIndex - The LOD we will apply the reduction settings.
	 * @param ReductionOptions - The reduction settings we want to apply to the LOD.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API void SetLodReductionSettings(UStaticMesh* StaticMesh, const int32 LodIndex, const FMeshReductionSettings& ReductionOptions);

	/**
	 * Copy the build options with the specified LOD build settings.
	 * @param StaticMesh - Mesh to process.
	 * @param LodIndex - The LOD we get the reduction settings.
	 * @param OutBuildOptions - The build settings where we copy the build options.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API void GetLodBuildSettings(const UStaticMesh* StaticMesh, const int32 LodIndex, FMeshBuildSettings& OutBuildOptions);

	/**
	 * Set the LOD build options for the specified LOD index.
	 * @param StaticMesh - Mesh to process.
	 * @param LodIndex - The LOD we will apply the build settings.
	 * @param BuildOptions - The build settings we want to apply to the LOD.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API void SetLodBuildSettings(UStaticMesh* StaticMesh, const int32 LodIndex, const FMeshBuildSettings& BuildOptions);

	/**
	 * Get the LODGroup for the specified static mesh
	 * @param StaticMesh
	 * @return LODGroup
	 */
	UFUNCTION(BlueprintCallable,Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API FName GetLODGroup(const UStaticMesh* StaticMesh);

	/**
	 * Set the LODGroup for the specified static mesh
	 * @param StaticMesh - Mesh to process.
	 * @param LODGroup - Name of the LODGroup to apply
	 * @param bRebuildImmediately - If true, rebuild the static mesh immediately
	 * @return Success
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API bool SetLODGroup(UStaticMesh* StaticMesh, FName LODGroup, bool bRebuildImmediately=true);
	
	/**
	 * Import or re-import a LOD into the specified base mesh. If the LOD do not exist it will import it and add it to the base static mesh. If the LOD already exist it will re-import the specified LOD.
	 *
	 * @param BaseStaticMesh: The static mesh we import or re-import a LOD.
	 * @param LODIndex: The index of the LOD to import or re-import. Valid value should be between 0 and the base static mesh LOD number. Invalid value will return INDEX_NONE
	 * @param SourceFilename: The fbx source filename. If we are re-importing an existing LOD, it can be empty in this case it will use the last import file. Otherwise it must be an existing fbx file.
	 *
	 * @return the index of the LOD that was imported or re-imported. Will return INDEX_NONE if anything goes bad.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API int32 ImportLOD(UStaticMesh* BaseStaticMesh, const int32 LODIndex, const FString& SourceFilename);

	/**
	 * Re-import all the custom LODs present in the specified static mesh.
	 *
	 * @param StaticMesh: is the static mesh we import or re-import all LODs.
	 *
	 * @return true if re-import all LODs works, false otherwise see log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh", meta=(ScriptName="ReimportAllCustomLods;ReimportAllCustomLODs"))
	STATICMESHEDITOR_API bool ReimportAllCustomLODs(UStaticMesh* StaticMesh);
	/**
	 * Adds or create a LOD at DestinationLodIndex using the geometry from SourceStaticMesh SourceLodIndex
	 * @param	DestinationStaticMesh		The static mesh to set the LOD in.
	 * @param	DestinationLodIndex			The index of the LOD to set.
	 * @param	SourceStaticMesh			The static mesh to get the LOD from.
	 * @param	SourceLodIndex				The index of the LOD to get.
	 * @param	bReuseExistingMaterialSlots	If true, sections from SourceStaticMesh will be remapped to match the material slots of DestinationStaticMesh
											when they have the same material assigned. If false, all material slots of SourceStaticMesh will be appended in DestinationStaticMesh.
	 * @return	The index of the LOD that was set. It can be different than DestinationLodIndex if it wasn't a valid index.
	 *			A negative value indicates that the LOD was not set. See log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 SetLodFromStaticMesh(UStaticMesh* DestinationStaticMesh, int32 DestinationLodIndex, UStaticMesh* SourceStaticMesh, int32 SourceLodIndex, bool bReuseExistingMaterialSlots);

	/**
	 * Get number of LODs present on a static mesh.
	 * @param	StaticMesh				Mesh to process.
	 * @return the number of LODs present on the input mesh.
	 * An negative value indicates that the command could not be executed. See log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetLodCount(const UStaticMesh* StaticMesh);

	/**
	 * Remove LODs on a static mesh except LOD 0.
	 * @param	StaticMesh			Mesh to remove LOD from.
	 * @return A boolean indicating if the removal was successful, true, or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool RemoveLods(UStaticMesh* StaticMesh);

	/**
	 * Get an array of LOD screen sizes for evaluation.
	 * @param	StaticMesh			Mesh to process.
	 * @return array of LOD screen sizes.
	 */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API TArray<float> GetLodScreenSizes(const UStaticMesh* StaticMesh);

	/**
	 * Set LOD screen sizes.
	 * @param	StaticMesh			Mesh to process.
	 * @param	ScreenSizes			Array of LOD screen sizes to set.
	 * @return A boolean indicating if setting the screen sizes was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool SetLodScreenSizes(UStaticMesh* StaticMesh, const TArray<float>& ScreenSizes);

public:

	/**
	 * Import or re-import the nanite hi-res mesh into the specified base mesh.
	 *
	 * @param StaticMesh:	The static mesh we import or re-import the nanite hi-res mesh.
	 * @param SourceFilename:	The source filename containing the hi-res mesh data. If we are re-importing an existing nanite hi-res mesh,
	 *							it can be empty, in this case it will use the last import file. Otherwise it must be an existing file.
	 * @param bShowDialogWhenFileMissing: If true a file picker will show in case the provided filename and the existing source filename are empty or do not exist.
	 *
	 * @return true if the nanite hi-res mesh import/re-import is successful, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API bool ImportNaniteHiResMesh(UStaticMesh* StaticMesh, const FString& SourceFilename, bool bShowDialogWhenFileMissing);

	/**
	* Get the Nanite hi-res source filename
	* @param StaticMesh        Mesh to access
	* @return the source filename use by the static mesh nanite hi-res.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API FString GetNaniteSourceFilename(const UStaticMesh* StaticMesh);

	/**
	* Update the Nanite hi-res source filename.
	* @param StaticMesh			Mesh to update the nanite source filename
	* @param NewSourceFilename	The source filename we want to set. If empty, imported nanite mesh will be remove. If not empty, the file must exist otherwise nothing will be done.
	* @return true if the source filename was updated (cleared or replace by an existing filename), false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | StaticMesh")
	STATICMESHEDITOR_API bool UpdateNaniteSourceFilename(UStaticMesh* StaticMesh, const FString& NewSourceFilename);

	/**
	* Get the Nanite Settings for the mesh
	* @param StaticMesh        Mesh to access
	* @return FMeshNaniteSettings struct for the given static mesh
	*/
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API FMeshNaniteSettings GetNaniteSettings(const UStaticMesh* StaticMesh);

	/**
	* Get the Nanite Settings for the mesh
	* @param StaticMesh        Mesh to update nanite settings for
	* @param NaniteSettings    Settings with which to update the mesh
	* @param bApplyChanges     Indicates if changes must be applied or not.
	*/
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void SetNaniteSettings(UStaticMesh* StaticMesh, FMeshNaniteSettings NaniteSettings, bool bApplyChanges=true);


	
public:
	/**
	 * Add simple collisions to a static mesh.
	 * This method replicates what is done when invoking menu entries "Collision > Add [...] Simplified Collision" in the Mesh Editor.
	 * @param	StaticMesh			Mesh to generate simple collision for.
	 * @param	ShapeType			Options on which simple collision to add to the mesh.
	 * @param	bApplyChanges		Indicates if changes must be apply or not.
	 * @return An integer indicating the index of the collision newly created.
	 * A negative value indicates the addition failed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 AddSimpleCollisionsWithNotification(UStaticMesh* StaticMesh, const EScriptCollisionShapeType ShapeType, bool bApplyChanges);

	/**
	 * Same as AddSimpleCollisionsWithNotification but changes are automatically applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	int32 AddSimpleCollisions(UStaticMesh* StaticMesh, const EScriptCollisionShapeType ShapeType)
	{
		return AddSimpleCollisionsWithNotification(StaticMesh, ShapeType, true);
	}

	/**
	 * Get number of simple collisions present on a static mesh.
	 * @param	StaticMesh				Mesh to query on.
	 * @return An integer representing the number of simple collisions on the input static mesh.
	 * An negative value indicates that the command could not be executed. See log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetSimpleCollisionCount(const UStaticMesh* StaticMesh);

	/**
	 * Get the Collision Trace behavior of a static mesh
	 * @param	StaticMesh				Mesh to query on.
	 * @return the Collision Trace behavior.
	 */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API TEnumAsByte<ECollisionTraceFlag> GetCollisionComplexity(const UStaticMesh* StaticMesh);

	/**
	 * Get number of convex collisions present on a static mesh.
	 * @param	StaticMesh				Mesh to query on.
	 * @return An integer representing the number of convex collisions on the input static mesh.
	 * An negative value indicates that the command could not be executed. See log for explanation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetConvexCollisionCount(const UStaticMesh* StaticMesh);

	/**
	 * Add a convex collision to a static mesh.
	 * Any existing collisions will be removed from the static mesh.
	 * This method replicates what is done when invoking menu entry "Collision > Auto Convex Collision" in the Mesh Editor.
	 * @param	StaticMesh				Static mesh to add convex collision on.
	 * @param	HullCount				Maximum number of convex pieces that will be created. Must be positive.
	 * @param	MaxHullVerts			Maximum number of vertices allowed for any generated convex hull.
	 * @param	HullPrecision			Number of voxels to use when generating collision. Must be positive.
	 * @param	bApplyChanges			Indicates if changes must be apply or not.
	 * @return A boolean indicating if the addition was successful or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool SetConvexDecompositionCollisionsWithNotification(UStaticMesh* StaticMesh, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, bool bApplyChanges);

	/**
	 * Compute convex collisions for a set of static meshes.
	 * Any existing collisions will be removed from the static meshes.
	 * This method replicates what is done when invoking menu entry "Collision > Auto Convex Collision" in the Mesh Editor.
	 * @param	StaticMeshes			Set of Static mesh to add convex collision on.
	 * @param	HullCount				Maximum number of convex pieces that will be created. Must be positive.
	 * @param	MaxHullVerts			Maximum number of vertices allowed for any generated convex hull.
	 * @param	HullPrecision			Number of voxels to use when generating collision. Must be positive.
	 * @param	bApplyChanges			Indicates if changes must be apply or not.
	 * @return A boolean indicating if the addition was successful or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool BulkSetConvexDecompositionCollisionsWithNotification(const TArray<UStaticMesh*>& StaticMeshes, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision, bool bApplyChanges);

	/**
	 * Same as SetConvexDecompositionCollisionsWithNotification but changes are automatically applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	bool SetConvexDecompositionCollisions(UStaticMesh* StaticMesh, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision)
	{
		return SetConvexDecompositionCollisionsWithNotification(StaticMesh, HullCount, MaxHullVerts, HullPrecision, true);
	}

	/**
	 * Same as SetConvexDecompositionCollisionsWithNotification but changes are automatically applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	bool BulkSetConvexDecompositionCollisions(const TArray<UStaticMesh*>& StaticMeshes, int32 HullCount, int32 MaxHullVerts, int32 HullPrecision)
	{
		return BulkSetConvexDecompositionCollisionsWithNotification(StaticMeshes, HullCount, MaxHullVerts, HullPrecision, true);
	}

	/**
	 * Remove collisions from a static mesh.
	 * This method replicates what is done when invoking menu entries "Collision > Remove Collision" in the Mesh Editor.
	 * @param	StaticMesh			Static mesh to remove collisions from.
	 * @param	bApplyChanges		Indicates if changes must be apply or not.
	 * @return A boolean indicating if the removal was successful or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool RemoveCollisionsWithNotification(UStaticMesh* StaticMesh, bool bApplyChanges);

	/**
	 * Same as RemoveCollisionsWithNotification but changes are applied.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	bool RemoveCollisions(UStaticMesh* StaticMesh)
	{
		return RemoveCollisionsWithNotification(StaticMesh, true);
	}

private:
	/**
	* Checks if section and lod indices are valid indices.
	*/
	static STATICMESHEDITOR_API bool CanModifyStaticMeshSection(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex, const FString& LogFunctionName);

public:
	/**
	 * Enables/disables mesh section shadow casting for a specific LOD.
	 * @param	StaticMesh			Static mesh to enable/disable section's shadow casting from.
	 * @param	bCastShadow			If the section should cast shadow.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void EnableSectionCastShadow(UStaticMesh* StaticMesh, bool bCastShadow, int32 LODIndex, int32 SectionIndex);

	/**
	 * Checks if a specific LOD mesh section can cast shadows.
	 * @param	StaticMesh			Static mesh to check if its section is casting shadows.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool IsSectionCastShadowEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);

	/**
	 * Enables/disables mesh section collision for a specific LOD.
	 * @param	StaticMesh			Static mesh to Enables/disables collisions from.
	 * @param	bCollisionEnabled	If the collision is enabled or not.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void EnableSectionCollision(UStaticMesh* StaticMesh, bool bCollisionEnabled, int32 LODIndex, int32 SectionIndex);

	/**
	 * Checks if a specific LOD mesh section has collision.
	 * @param	StaticMesh			Static mesh to check section collisions from.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 * @return True if the collision is enabled for the specified LOD of the StaticMesh section.
	 */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool IsSectionCollisionEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);

	/**
	 * Enables/disables mesh section's visibility in ray tracing for a specific LOD.
	 * @param	StaticMesh				Static mesh to enable/disable section's visibility in ray tracing from.
	 * @param	bVisibleInRayTracing	If the section visible in ray tracing.
	 * @param	LODIndex				Index of the StaticMesh LOD.
	 * @param	SectionIndex			Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void EnableSectionVisibleInRayTracing(UStaticMesh* StaticMesh, bool bVisibleInRayTracing, int32 LODIndex, int32 SectionIndex);

	/**
	 * Checks if a specific LOD mesh section is visible in ray tracing.
	 * @param	StaticMesh			Static mesh to check if its section is visible in ray tracing.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool IsSectionVisibleInRayTracingEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);

	/**
	 * Enables/disables mesh section affects distance field lighting for a specific LOD.
	 * @param	StaticMesh						Static mesh to enable/disable section affecting distance field lighting from.
	 * @param	bAffectDistanceFieldLighting	If the section affects distance field lighting.
	 * @param	LODIndex						Index of the StaticMesh LOD.
	 * @param	SectionIndex					Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void EnableSectionAffectDistanceFieldLighting(UStaticMesh* StaticMesh, bool bAffectDistanceFieldLighting, int32 LODIndex, int32 SectionIndex);

	/**
	 * Checks if a specific LOD mesh section affects distance field lighting.
	 * @param	StaticMesh			Static mesh to check if its section affects distance field lighting.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool IsSectionAffectDistanceFieldLightingEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);


	/**
	 * Set/unset mesh section to be forcefully opaque for a specific LOD.
	 * @param	StaticMesh			Static mesh to make section forcefully opaque from.
	 * @param	bForceOpaque		If the section is forced to be opaque.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void EnableSectionForceOpaque(UStaticMesh* StaticMesh, bool bForceOpaque, int32 LODIndex, int32 SectionIndex);

	/**
	 * Checks if a specific LOD mesh section is forced to be opaque.
	 * @param	StaticMesh			Static mesh to is forced to be opaque.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool IsSectionForceOpaqueEnabled(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);

public:
	/**
	* Sets the material slot for a specific LOD.
	* @param	StaticMesh			Static mesh to Enables/disables shadow casting from.
	* @param	MaterialSlotIndex	Index of the material slot to use.
	* @param	LODIndex			Index of the StaticMesh LOD.
	* @param	SectionIndex		Index of the StaticMesh Section.
	*/
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void SetLODMaterialSlot(UStaticMesh* StaticMesh, int32 MaterialSlotIndex, int32 LODIndex, int32 SectionIndex);

	/**
	 * Gets the material slot used for a specific LOD section.
	 * @param	StaticMesh			Static mesh to get the material index from.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	SectionIndex		Index of the StaticMesh Section.
	 * @return  MaterialSlotIndex	Index of the material slot used by the section or INDEX_NONE in case of error.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetLODMaterialSlot(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex);

	/** Check whether a static mesh has vertex colors */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool HasVertexColors(const UStaticMesh* StaticMesh);

	/** Check whether a static mesh component has vertex colors */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool HasInstanceVertexColors(const UStaticMeshComponent* StaticMeshComponent);

	/** Set Generate Lightmap UVs for StaticMesh */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities", meta = (ScriptName = "SetGenerateLightmapUv"))
	STATICMESHEDITOR_API bool SetGenerateLightmapUVs(UStaticMesh* StaticMesh, bool bGenerateLightmapUVs);

	/** Get number of StaticMesh verts for an LOD */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetNumberVerts(const UStaticMesh* StaticMesh, int32 LODIndex);

	/** Get number of Materials for a StaticMesh */
	UFUNCTION(BlueprintPure, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetNumberMaterials(const UStaticMesh* StaticMesh);

	/** Sets StaticMeshFlag bAllowCPUAccess  */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void SetAllowCPUAccess(UStaticMesh* StaticMesh, bool bAllowCPUAccess);

public:

	/**
	 * Returns the number of UV channels for the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh to query.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @return the number of UV channels.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API int32 GetNumUVChannels(const UStaticMesh* StaticMesh, int32 LODIndex);

	/**
	 * Adds an empty UV channel at the end of the existing channels on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to add a UV channel.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @return true if a UV channel was added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool AddUVChannel(UStaticMesh* StaticMesh, int32 LODIndex);

	/**
	 * Inserts an empty UV channel at the specified channel index on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to insert a UV channel.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to insert the UV channel.
	 * @return true if a UV channel was added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool InsertUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex);

	/**
	 * Removes the UV channel at the specified channel index on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to remove the UV channel.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Index where to remove the UV channel.
	 * @return true if the UV channel was removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool RemoveUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex);

	/**
	 * Generates planar UV mapping in the specified UV channel on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to generate the UV mapping.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Channel where to save the UV mapping.
	 * @param	Position			Position of the center of the projection gizmo.
	 * @param	Orientation			Rotation to apply to the projection gizmo.
	 * @param	Tiling				The UV tiling to use to generate the UV mapping.
	 * @return true if the UV mapping was generated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool GeneratePlanarUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector2D& Tiling);

	/**
	 * Generates cylindrical UV mapping in the specified UV channel on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to generate the UV mapping.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Channel where to save the UV mapping.
	 * @param	Position			Position of the center of the projection gizmo.
	 * @param	Orientation			Rotation to apply to the projection gizmo.
	 * @param	Tiling				The UV tiling to use to generate the UV mapping.
	 * @return true if the UV mapping was generated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool GenerateCylindricalUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector2D& Tiling);

	/**
	 * Generates box UV mapping in the specified UV channel on the given LOD of a StaticMesh.
	 * @param	StaticMesh			Static mesh on which to generate the UV mapping.
	 * @param	LODIndex			Index of the StaticMesh LOD.
	 * @param	UVChannelIndex		Channel where to save the UV mapping.
	 * @param	Position			Position of the center of the projection gizmo.
	 * @param	Orientation			Rotation to apply to the projection gizmo.
	 * @param	Size				The size of the box projection gizmo.
	 * @return true if the UV mapping was generated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API bool GenerateBoxUVChannel(UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannelIndex, const FVector& Position, const FRotator& Orientation, const FVector& Size);

	/**
	 * Find the references of the material MaterialToReplaced on all the MeshComponents provided and replace it by NewMaterial.
	 * @param	MeshComponents			List of MeshComponent to search from.
	 * @param	MaterialToBeReplaced	Material we want to replace.
	 * @param	NewMaterial				Material to replace MaterialToBeReplaced by.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void ReplaceMeshComponentsMaterials(const TArray<class UMeshComponent*>& MeshComponents, class UMaterialInterface* MaterialToBeReplaced, class UMaterialInterface* NewMaterial);

	/**
	 * Find the references of the material MaterialToReplaced on all the MeshComponents of all the Actors provided and replace it by NewMaterial.
	 * @param	Actors					List of Actors to search from.
	 * @param	MaterialToBeReplaced	Material we want to replace.
	 * @param	NewMaterial				Material to replace MaterialToBeReplaced by.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void ReplaceMeshComponentsMaterialsOnActors(const TArray<class AActor*>& Actors, class UMaterialInterface* MaterialToBeReplaced, class UMaterialInterface* NewMaterial);

	/**
	 * Find the references of the mesh MeshToBeReplaced on all the MeshComponents provided and replace it by NewMesh.
	 * The editor should not be in play in editor mode.
	 * @param	MeshComponents			List of MeshComponent to search from.
	 * @param	MeshToBeReplaced		Mesh we want to replace.
	 * @param	NewMesh					Mesh to replace MeshToBeReplaced by.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void ReplaceMeshComponentsMeshes(const TArray<class UStaticMeshComponent*>& MeshComponents, class UStaticMesh* MeshToBeReplaced, class UStaticMesh* NewMesh);

	/**
	 * Find the references of the mesh MeshToBeReplaced on all the MeshComponents of all the Actors provided and replace it by NewMesh.
	 * @param	Actors					List of Actors to search from.
	 * @param	MeshToBeReplaced		Mesh we want to replace.
	 * @param	NewMesh					Mesh to replace MeshToBeReplaced by.
	 */
	UFUNCTION(BlueprintCallable, Category = "Static Mesh Utilities")
	STATICMESHEDITOR_API void ReplaceMeshComponentsMeshesOnActors(const TArray<class AActor*>& Actors, class UStaticMesh* MeshToBeReplaced, class UStaticMesh* NewMesh);

		/**
	 * Create a new Actor in the level that contains a duplicate of all the Actors Static Meshes Component.
	 * The ActorsToJoin need to be in the same Level.
	 * This will have a low impact on performance but may help the edition by grouping the meshes under a single Actor.
	 * @param	ActorsToJoin			List of Actors to join.
	 * @param	JoinOptions				Options on how to join the actors.
	 * @return The new created actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep")
	STATICMESHEDITOR_API class AActor* JoinStaticMeshActors(const TArray<class AStaticMeshActor*>& ActorsToJoin, const FJoinStaticMeshActorsOptions& JoinOptions);

	/**
	 * Merge the meshes into a unique mesh with the provided StaticMeshActors. There are multiple options on how to merge the meshes and their materials.
	 * The ActorsToMerge need to be in the same Level.
	 * This may have a high impact on performance depending of the MeshMergingSettings options.
	 * @param	ActorsToMerge			List of Actors to merge.
	 * @param	MergeOptions			Options on how to merge the actors.
	 * @param	OutMergedActor			The new created actor, if requested.
	 * @return	if the operation is successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep")
	STATICMESHEDITOR_API bool MergeStaticMeshActors(const TArray<class AStaticMeshActor*>& ActorsToMerge, const FMergeStaticMeshActorsOptions& MergeOptions, class AStaticMeshActor*& OutMergedActor);

	/**
	 * Build a proxy mesh actor that can replace a set of mesh actors.
	 * @param   ActorsToMerge  List of actors to build a proxy for.
	 * @param   MergeOptions
	 * @param   OutMergedActor generated actor if requested
	 * @return  Success of the proxy creation
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Dataprep")
	STATICMESHEDITOR_API bool CreateProxyMeshActor(const TArray<class AStaticMeshActor*>& ActorsToMerge, const FCreateProxyMeshActorOptions& MergeOptions, class AStaticMeshActor*& OutMergedActor);
};
