// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BodySetupEnums.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"
#include "UVMapSettings.h"
#include "StaticMeshEditorSubsystemHelpers.generated.h"

class UStaticMeshComponent;

USTRUCT(BlueprintType)
struct FStaticMeshReductionSettings
{
	GENERATED_BODY()

	FStaticMeshReductionSettings()
		: PercentTriangles(0.5f)
	    , ScreenSize(0.5f)
	{ }

	// Percentage of triangles to keep. Ranges from 0.0 to 1.0: 1.0 = no reduction, 0.0 = no triangles.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float PercentTriangles;

	// ScreenSize to display this LOD. Ranges from 0.0 to 1.0.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	float ScreenSize;
};

USTRUCT(BlueprintType)
struct FStaticMeshReductionOptions
{
	GENERATED_BODY()

	FStaticMeshReductionOptions()
		: bAutoComputeLODScreenSize(true)
	{ }

	// If true, the screen sizes at which LODs swap are computed automatically
	// @note that this is displayed as 'Auto Compute LOD Distances' in the UI
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bAutoComputeLODScreenSize;

	// Array of reduction settings to apply to each new LOD mesh.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	TArray<FStaticMeshReductionSettings> ReductionSettings;
};

/** Types of Collision Construct that are generated **/
UENUM(BlueprintType)
enum class EScriptCollisionShapeType : uint8
{
	Box,
	Sphere,
	Capsule,
	NDOP10_X,
	NDOP10_Y,
	NDOP10_Z,
	NDOP18,
	NDOP26
};


USTRUCT(BlueprintType)
struct FJoinStaticMeshActorsOptions
{
	GENERATED_BODY()

	FJoinStaticMeshActorsOptions()
		: bDestroySourceActors(true)
		, bRenameComponentsFromSource(true)
	{ }

	// Destroy the provided Actors after the operation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bDestroySourceActors;

	// Name of the new spawned Actor to replace the provided Actors.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FString NewActorLabel;

	// Rename StaticMeshComponents based on source Actor's name.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bRenameComponentsFromSource;
};

USTRUCT(BlueprintType)
struct FMergeStaticMeshActorsOptions : public FJoinStaticMeshActorsOptions
{
	GENERATED_BODY()

	FMergeStaticMeshActorsOptions()
		: bSpawnMergedActor(true)
	{ }

	// Spawn the new merged actors
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSpawnMergedActor;

	// The package path you want to save to. ie: /Game/MyFolder
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FString BasePackageName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FMeshMergingSettings MeshMergingSettings;
};

USTRUCT(BlueprintType)
struct FCreateProxyMeshActorOptions : public FJoinStaticMeshActorsOptions
{
	GENERATED_BODY()

	FCreateProxyMeshActorOptions()
		: bSpawnMergedActor(true)
	{ }

	// Spawn the new merged actors
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	bool bSpawnMergedActor;

	// The package path you want to save to. ie: /Game/MyFolder
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FString BasePackageName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Options)
	FMeshProxySettings MeshProxySettings;
};
