// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StaticMeshEditor : ModuleRules
{
	public StaticMeshEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(
			new string[] {
				System.IO.Path.Combine(GetModuleDirectory("UnrealEd"), "Private"),
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "MeshReductionInterface",
            }
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"RenderCore",
				"RHI",
				"EditorFramework",
				"UnrealEd",
				"TargetPlatform",
				"RawMesh",
                "PropertyEditor",
				"MeshUtilities",
				"NaniteUtilities",
                "Json",
                "JsonUtilities",
                "AdvancedPreviewScene",
                "DesktopPlatform",
                "DesktopWidgets",
				"EditorSubsystem",
				"MeshDescription",
				"StaticMeshDescription",
				"CommonMenuExtensions",
				"ToolMenus",
				"DetailCustomizations",
				"StatusBar",
				"WorkspaceMenuStructure",
				"PhysicsUtilities",
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"SceneOutliner",
				"ClassViewer",
				"ContentBrowser",
                "MeshReductionInterface",
            }
		);

		SetupModulePhysicsSupport(Target);
	}
}
